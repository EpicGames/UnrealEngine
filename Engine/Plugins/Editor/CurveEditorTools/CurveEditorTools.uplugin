{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Curve Editor Tools",
	"Description": "This provides a default set of editing tools for the Curve Editor.",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"EnabledByDefault": true,
	"Plugins": [
		{
			"Name": "TweeningUtils",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "CurveEditorTools",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}