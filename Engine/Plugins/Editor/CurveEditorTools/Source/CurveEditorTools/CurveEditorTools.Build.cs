// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CurveEditorTools : ModuleRules
{
	public CurveEditorTools(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CurveEditor",
				"Engine",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"InputCore",
				"SlateCore",
				"Slate",
				"SequencerWidgets",
				"EditorFramework",
				"UnrealEd",
				"CoreUObject",
				"AudioMixer",
				"SignalProcessing",
				"EditorStyle", 
				"TweeningUtils",
				"TweeningUtilsEditor",
			}
		);
	}
}