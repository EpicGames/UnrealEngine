// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorToolCommands.h"

#define LOCTEXT_NAMESPACE "CurveEditorToolCommands"

void FCurveEditorToolCommands::RegisterCommands()
{
	// Focus Tools
	UI_COMMAND(SetFocusPlaybackTime, "Focus Playback Time", "Focuses the Curve Editor on the current Playback Time without changing zoom level.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SetFocusPlaybackRange, "Focus Playback Range", "Focuses the Curve Editor on the current Playback Range with zoom based on visible curves.", EUserInterfaceActionType::Button, FInputChord(EKeys::A));
	UI_COMMAND(SetFocusSelectionRange, "Focus Selection Range", "Focuses the Curve Editor on the current Selection Range with zoom based on visible curves.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SetKeysInSelectionRange, "Select Keys in Selection Range",
		"Selects all keys within the current Selection Range (or Playback Range if no selection range is set).",
		EUserInterfaceActionType::Button, FInputChord());

	// Tool Modes
	UI_COMMAND(ActivateTransformTool, "Transform", "The Transform tool allows translation, scale and rotation of selected keys.", EUserInterfaceActionType::RadioButton, FInputChord(EModifierKey::Control, EKeys::T));
	UI_COMMAND(ActivateRetimeTool, "Retime", "The Retime tool allows you to define a one dimensional lattice to non-uniformly rescale key times.", EUserInterfaceActionType::RadioButton, FInputChord(EModifierKey::Control, EKeys::E));
	UI_COMMAND(ActivateMultiScaleTool, "Multi Scale", "The Multi Select tool allows fast scaling of multiple curves from unique pivots.", EUserInterfaceActionType::RadioButton, FInputChord(EModifierKey::Control, EKeys::M));
	UI_COMMAND(ActivateLatticeTool, "Lattice", "The Lattice tool allows fast editing of keys using a grid.", EUserInterfaceActionType::RadioButton, FInputChord(EModifierKey::Control, EKeys::L));
}

#undef LOCTEXT_NAMESPACE
