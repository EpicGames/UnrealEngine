// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/AppStyle.h"
#include "Framework/Commands/Commands.h"

/**
 * Defines commands for the Curve Editor Tools plugin which enables most functionality of the Curve Editor.
 */
class CURVEEDITORTOOLS_API FCurveEditorToolCommands : public TCommands<FCurveEditorToolCommands>
{
public:
	FCurveEditorToolCommands()
		: TCommands<FCurveEditorToolCommands>
		(
			TEXT("CurveEditorTools"),
			NSLOCTEXT("Contexts", "CurveEditorTools", "Curve Editor Tools"),
			NAME_None,
			FAppStyle::GetAppStyleSetName()
		)
	{
	}

	TSharedPtr<FUICommandInfo> SetFocusPlaybackTime;
	TSharedPtr<FUICommandInfo> SetFocusPlaybackRange;
	TSharedPtr<FUICommandInfo> SetFocusSelectionRange;
	TSharedPtr<FUICommandInfo> SetKeysInSelectionRange;
	TSharedPtr<FUICommandInfo> ActivateTransformTool;
	TSharedPtr<FUICommandInfo> ActivateRetimeTool;
	TSharedPtr<FUICommandInfo> ActivateMultiScaleTool;
	TSharedPtr<FUICommandInfo> ActivateLatticeTool;

public:
	virtual void RegisterCommands() override;
};
