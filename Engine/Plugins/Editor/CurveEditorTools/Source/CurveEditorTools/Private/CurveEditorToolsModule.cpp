// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorHelpers.h"
#include "Modules/ModuleManager.h"
#include "Framework/Commands/UICommandList.h"
#include "ICurveEditorModule.h"
#include "Extensions/CurveEditorFocusExtension.h"
#include "Extensions/Tweening/TweenEditorExtension.h"
#include "CurveEditorToolCommands.h"
#include "Tools/CurveEditorTransformTool.h"
#include "Tools/Retime/CurveEditorRetimeTool.h"
#include "Tools/CurveEditorMultiScaleTool.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Tools/Lattice/CurveEditorLatticeTool.h"

#define LOCTEXT_NAMESPACE "CurveEditorToolsModule"

class FCurveEditorToolsModule : public IModuleInterface, public TSharedFromThis<FCurveEditorToolsModule>
{
	// IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// ~IModuleInterface

private:
	static TSharedRef<ICurveEditorExtension> CreateFocusExtension(TWeakPtr<FCurveEditor> InCurveEditor);
	static TSharedRef<ICurveEditorExtension> CreateTweenExtension(TWeakPtr<FCurveEditor> InCurveEditor);
	static TUniquePtr<ICurveEditorToolExtension> CreateTransformToolExtension(TWeakPtr<FCurveEditor> InCurveEditor);
	static TUniquePtr<ICurveEditorToolExtension> CreateRetimeToolExtension(TWeakPtr<FCurveEditor> InCurveEditor);
	static TUniquePtr<ICurveEditorToolExtension> CreateMultiScaleToolExtension(TWeakPtr<FCurveEditor> InCurveEditor);
	static TUniquePtr<ICurveEditorToolExtension> CreateLatticeToolExtensions(TWeakPtr<FCurveEditor> InCurveEditor);

protected:
	TSharedRef<FExtender> ExtendCurveEditorToolbarMenu(const TSharedRef<FUICommandList> CommandList);


private:
	FDelegateHandle FocusExtensionsHandle;
	FDelegateHandle TweenExtensionHandle;
	
	FDelegateHandle TransformToolHandle;
	FDelegateHandle RetimeToolHandle;
	FDelegateHandle MultiScaleToolHandle;
	FDelegateHandle LatticeToolHandle;
};

IMPLEMENT_MODULE(FCurveEditorToolsModule, CurveEditorTools)

void FCurveEditorToolsModule::StartupModule()
{
	FCurveEditorToolCommands::Register();

	ICurveEditorModule& CurveEditorModule = FModuleManager::Get().LoadModuleChecked<ICurveEditorModule>("CurveEditor");

	// Register Editor Extensions
	FocusExtensionsHandle = CurveEditorModule.RegisterEditorExtension(FOnCreateCurveEditorExtension::CreateStatic(&FCurveEditorToolsModule::CreateFocusExtension));
	TweenExtensionHandle = CurveEditorModule.RegisterEditorExtension(FOnCreateCurveEditorExtension::CreateStatic(&FCurveEditorToolsModule::CreateTweenExtension));

	// Register Tool Extensions
	TransformToolHandle = CurveEditorModule.RegisterToolExtension(FOnCreateCurveEditorToolExtension::CreateStatic(&FCurveEditorToolsModule::CreateTransformToolExtension));
	RetimeToolHandle = CurveEditorModule.RegisterToolExtension(FOnCreateCurveEditorToolExtension::CreateStatic(&FCurveEditorToolsModule::CreateRetimeToolExtension));
	MultiScaleToolHandle = CurveEditorModule.RegisterToolExtension(FOnCreateCurveEditorToolExtension::CreateStatic(&FCurveEditorToolsModule::CreateMultiScaleToolExtension));
	LatticeToolHandle = CurveEditorModule.RegisterToolExtension(FOnCreateCurveEditorToolExtension::CreateStatic(&FCurveEditorToolsModule::CreateLatticeToolExtensions));

	auto ToolbarExtender = ICurveEditorModule::FCurveEditorMenuExtender::CreateRaw(this, &FCurveEditorToolsModule::ExtendCurveEditorToolbarMenu);
	auto& MenuExtenders = CurveEditorModule.GetAllToolBarMenuExtenders();
	MenuExtenders.Add(ToolbarExtender);
}

TSharedRef<FExtender> FCurveEditorToolsModule::ExtendCurveEditorToolbarMenu(const TSharedRef<FUICommandList> CommandList)
{
	struct Local
	{
		static void FillToolbarTools(FMenuBuilder& ToolbarBuilder)
		{
			ToolbarBuilder.AddMenuEntry(FCurveEditorToolCommands::Get().ActivateTransformTool);
			ToolbarBuilder.AddMenuEntry(FCurveEditorToolCommands::Get().ActivateRetimeTool);
			ToolbarBuilder.AddMenuEntry(FCurveEditorToolCommands::Get().ActivateMultiScaleTool);
			ToolbarBuilder.AddMenuEntry(FCurveEditorToolCommands::Get().ActivateLatticeTool);
		}

		static void FillToolbarFraming(FMenuBuilder& ToolbarBuilder)
		{
			ToolbarBuilder.AddMenuEntry(FCurveEditorToolCommands::Get().SetFocusPlaybackTime);
			ToolbarBuilder.AddMenuEntry(FCurveEditorToolCommands::Get().SetFocusPlaybackRange);
		}
	};

	const TSharedRef<FExtender> Extender = MakeShared<FExtender>();
	Extender->AddMenuExtension("Tools", EExtensionHook::After, CommandList, FMenuExtensionDelegate::CreateStatic(&Local::FillToolbarTools));
	Extender->AddMenuExtension("Framing", EExtensionHook::After, CommandList, FMenuExtensionDelegate::CreateStatic(&Local::FillToolbarFraming));

	return Extender;
}
void FCurveEditorToolsModule::ShutdownModule()
{
	ICurveEditorModule& CurveEditorModule = FModuleManager::Get().LoadModuleChecked<ICurveEditorModule>("CurveEditor");

	// Unregister Editor Extensions
	CurveEditorModule.UnregisterEditorExtension(FocusExtensionsHandle);
	CurveEditorModule.UnregisterEditorExtension(TweenExtensionHandle);

	// Unregister Tool Extensions
	CurveEditorModule.UnregisterToolExtension(TransformToolHandle);
	CurveEditorModule.UnregisterToolExtension(RetimeToolHandle);
	CurveEditorModule.UnregisterToolExtension(MultiScaleToolHandle);
	CurveEditorModule.UnregisterToolExtension(LatticeToolHandle);
	
	FCurveEditorToolCommands::Unregister();
}

TSharedRef<ICurveEditorExtension> FCurveEditorToolsModule::CreateFocusExtension(TWeakPtr<FCurveEditor> InCurveEditor) 
{
	return MakeShared<FCurveEditorFocusExtension>(InCurveEditor);
}

TSharedRef<ICurveEditorExtension> FCurveEditorToolsModule::CreateTweenExtension(TWeakPtr<FCurveEditor> InCurveEditor)
{
	return MakeShared<UE::CurveEditorTools::FTweenEditorExtension>(InCurveEditor);
}

TUniquePtr<ICurveEditorToolExtension> FCurveEditorToolsModule::CreateTransformToolExtension(TWeakPtr<FCurveEditor> InCurveEditor)
{
	return MakeUnique<FCurveEditorTransformTool>(InCurveEditor);
}

TUniquePtr<ICurveEditorToolExtension> FCurveEditorToolsModule::CreateRetimeToolExtension(TWeakPtr<FCurveEditor> InCurveEditor)
{
	return MakeUnique<UE::CurveEditorTools::FCurveEditorRetimeTool>(InCurveEditor);
}

TUniquePtr<ICurveEditorToolExtension> FCurveEditorToolsModule::CreateMultiScaleToolExtension(TWeakPtr<FCurveEditor> InCurveEditor)
{
	return MakeUnique<FCurveEditorMultiScaleTool>(InCurveEditor);
}

TUniquePtr<ICurveEditorToolExtension> FCurveEditorToolsModule::CreateLatticeToolExtensions(TWeakPtr<FCurveEditor> InCurveEditor)
{
	return MakeUnique<UE::CurveEditorTools::FCurveEditorLatticeTool>(InCurveEditor);
}

#undef LOCTEXT_NAMESPACE
