// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorFocusExtension.h"
#include "Framework/Commands/UICommandList.h"
#include "CurveEditorToolCommands.h"
#include "CurveEditor.h"
#include "ICurveEditorBounds.h"
#include "AnimatedRange.h"
#include "CurveModel.h"
#include "CurveEditorSelection.h"

void FCurveEditorFocusExtension::BindCommands(TSharedRef<FUICommandList> CommandBindings)
{
	// ToDo: Focus on Timeline is only valid if there is a externally provided time controller.
	FCanExecuteAction CanFocus = FCanExecuteAction::CreateSP(this, &FCurveEditorFocusExtension::CanUseFocusExtensions);
	FExecuteAction FocusPlaybackTime = FExecuteAction::CreateSP(this, &FCurveEditorFocusExtension::FocusPlaybackTime);
	FExecuteAction FocusPlaybackRange = FExecuteAction::CreateSP(this, &FCurveEditorFocusExtension::FocusPlaybackRange);
	FExecuteAction FocusSelectionRange = FExecuteAction::CreateSP(this, &FCurveEditorFocusExtension::FocusSelectionRange);

	CommandBindings->MapAction(FCurveEditorToolCommands::Get().SetFocusPlaybackTime, FocusPlaybackTime, CanFocus);
	CommandBindings->MapAction(FCurveEditorToolCommands::Get().SetFocusPlaybackRange, FocusPlaybackRange, CanFocus);
	CommandBindings->MapAction(FCurveEditorToolCommands::Get().SetFocusSelectionRange, FocusSelectionRange, CanFocus);

	FExecuteAction SelectKeysInSelectionRangeAction = FExecuteAction::CreateSP(
		this, &FCurveEditorFocusExtension::SelectKeysInSelectionRange);
	CommandBindings->MapAction(
		FCurveEditorToolCommands::Get().SetKeysInSelectionRange,
		SelectKeysInSelectionRangeAction, CanFocus);
}

void FCurveEditorFocusExtension::FocusPlaybackRange()
{
	// This changes the extents of your view range to focus on the current playback range of the curve editor.
	// This will change both zoom and position.
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor)
	{
		TSharedPtr<ITimeSliderController> TimeSliderController = CurveEditor->GetTimeSliderController();
		if (TimeSliderController)
		{
			TRange<FFrameNumber> PlaybackRange = TimeSliderController->GetPlayRange();

			// Tell the curve editor to Zoom to Fit on the Y axis. This will handle zooming in on the correct curves
			// before we adjust the represented horizontal range.
			CurveEditor->ZoomToFit(EAxisList::Type::Y);

			// Now we need to convert between Time Slider Controller's playback range and the bounds.
			FFrameNumber InFrame = TimeSliderController->GetPlayRange().GetLowerBoundValue();
			FFrameNumber OutFrame = TimeSliderController->GetPlayRange().GetUpperBoundValue();

			double InTime = InFrame / TimeSliderController->GetTickResolution();
			double OutTime = OutFrame / TimeSliderController->GetTickResolution();

			// Add 5% padding to each side
			double Padding = (OutTime - InTime) * 0.05;
			InTime -= Padding;
			OutTime += Padding;

			CurveEditor->GetBounds().SetInputBounds(InTime, OutTime);
		}
	}
}

void FCurveEditorFocusExtension::FocusSelectionRange()
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor)
	{
		TSharedPtr<ITimeSliderController> TimeSliderController = CurveEditor->GetTimeSliderController();
		if (TimeSliderController)
		{
			TRange<FFrameNumber> Range = TimeSliderController->GetSelectionRange();
			if (Range.IsEmpty())
			{
				Range = TimeSliderController->GetPlayRange();
			}

			CurveEditor->ZoomToFit(EAxisList::Type::Y);

			FFrameNumber InFrame  = Range.GetLowerBoundValue();
			FFrameNumber OutFrame = Range.GetUpperBoundValue();
			double InTime  = InFrame  / TimeSliderController->GetTickResolution();
			double OutTime = OutFrame / TimeSliderController->GetTickResolution();

			double Padding = (OutTime - InTime) * 0.05;
			InTime  -= Padding;
			OutTime += Padding;

			CurveEditor->GetBounds().SetInputBounds(InTime, OutTime);
		}
	}
}

void FCurveEditorFocusExtension::FocusPlaybackTime()
{
	// This takes your current view range (both x and y) and shifts it to be centered on the playback head.
	// This does not change the extents of your view range so the zoom level will not shift.
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor)
	{
		TSharedPtr<ITimeSliderController> TimeSliderController = CurveEditor->GetTimeSliderController();
		if (TimeSliderController)
		{
			double InTime, OutTime;
			CurveEditor->GetBounds().GetInputBounds(InTime, OutTime);

			// Calculate our total range
			double ViewRange = OutTime - InTime;

			const FFrameTime ScrubFrameTime = TimeSliderController->GetScrubPosition();

			// We cant move outside clamp range, so take that into account
			FAnimatedRange ClampRange = TimeSliderController->GetClampRange();

			// Now center it around the Time Slider.
			double PlaybackPosition = ScrubFrameTime / TimeSliderController->GetTickResolution();
			InTime = PlaybackPosition - (ViewRange / 2.0);

			// Take clamp range into account
			if(InTime < ClampRange.GetLowerBoundValue())
			{
				InTime = ClampRange.GetLowerBoundValue();
				OutTime = InTime + ViewRange;
			}
			else
			{
				OutTime = PlaybackPosition + (ViewRange / 2.0);
				if(OutTime > ClampRange.GetUpperBoundValue())
				{
					OutTime = ClampRange.GetUpperBoundValue();
					InTime = OutTime - ViewRange;
				}
			}

			CurveEditor->GetBounds().SetInputBounds(InTime, OutTime);
		}
	}
}

void FCurveEditorFocusExtension::SelectKeysInSelectionRange()
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return;
	}

	TSharedPtr<ITimeSliderController> TimeSliderController =
		CurveEditor->GetTimeSliderController();
	if (!TimeSliderController)
	{
		return;
	}

	// Resolve the time range: selection range first, play range as fallback
	TRange<FFrameNumber> FrameRange = TimeSliderController->GetSelectionRange();
	if (FrameRange.IsEmpty())
	{
		FrameRange = TimeSliderController->GetPlayRange();
	}
	if (FrameRange.IsEmpty())
	{
		return;
	}

	FFrameRate TickResolution = TimeSliderController->GetTickResolution();
	double InTime  = FrameRange.GetLowerBoundValue() / TickResolution;
	double OutTime = FrameRange.GetUpperBoundValue() / TickResolution;

	// Collect keys that fall within the range for every edited curve
	TMap<FCurveModelID, TArray<FKeyHandle>> KeysPerCurve;
	for (FCurveModelID ID : CurveEditor->GetEditedCurves())
	{
		if (FCurveModel* Curve = CurveEditor->FindCurve(ID))
		{
			TArray<FKeyHandle> KeyHandles;
			Curve->GetKeys(InTime, OutTime,
				TNumericLimits<double>::Lowest(), TNumericLimits<double>::Max(),
				KeyHandles);
			if (KeyHandles.Num() > 0)
			{
				KeysPerCurve.Add(ID, MoveTemp(KeyHandles));
			}
		}
	}

	// Only modify selection if at least one key was found
	if (KeysPerCurve.Num() > 0)
	{
		FCurveEditorSelection& Selection = CurveEditor->GetSelection();
		Selection.Clear();
		for (auto& Pair : KeysPerCurve)
		{
			Selection.Add(Pair.Key, ECurvePointType::Key, Pair.Value);
		}
	}
}

bool FCurveEditorFocusExtension::CanUseFocusExtensions() const
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor)
	{
		return CurveEditor->GetTimeSliderController().IsValid();
	}

	return false;
}
