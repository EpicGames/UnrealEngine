// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ICurveEditorExtension.h"

class FCurveEditor;

class FCurveEditorFocusExtension : public ICurveEditorExtension, public TSharedFromThis<FCurveEditorFocusExtension>
{
public:
	FCurveEditorFocusExtension(TWeakPtr<FCurveEditor> InCurveEditor)
		: WeakCurveEditor(InCurveEditor)
	{
	}

	// ICurveEditorExtension Interface
	virtual void BindCommands(TSharedRef<FUICommandList> CommandBindings) override;
	// ~ICurveEditorExtension

private:
	/** Focuses the curve editor around the current Playback Range of the linked Time Controller. */
	void FocusPlaybackRange();
	/** Focuses the curve editor around the current Selection Range of the linked Time Controller (falls back to Playback Range if empty). */
	void FocusSelectionRange();
	/** Selects all keys within the current Selection Range (or Playback Range if no selection range is set). */
	void SelectKeysInSelectionRange();
	/** Focuses the curve editor around the current Playback Time of the linked Time Controller without changing zoom level. */
	void FocusPlaybackTime();

	/** Checks to see if the Curve Editor has a valid Time Controller to use these focus extensions on. */
	bool CanUseFocusExtensions() const;
private:
	TWeakPtr<FCurveEditor> WeakCurveEditor;
};