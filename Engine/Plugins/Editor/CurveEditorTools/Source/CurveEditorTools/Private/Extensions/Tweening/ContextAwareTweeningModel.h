// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Models/CurveTweenModel.h"

#include <type_traits>

#include "KeyInsertingTweenModel.h"

class FCurveEditor;
template<typename ObjectType> class TAttribute;

namespace UE::CurveEditorTools
{
/**
 * This model tweens keys depending on the user selection:
 * - When keys are selected, FCurveTweenModel will tween the selected keys.
 * - When no keys are selected, we'll insert keys at the time controller's scrub position to all visible curves (those selected in tree view).
 *   If there's no such controller, nothing happens.
 */
template<TweeningUtilsEditor::EBlendFunction BlendFunction>
class TContextAwareTweeningModel : public TweeningUtilsEditor::FTweenModel
{
	using Super = FTweenModel;
public:
	
	explicit TContextAwareTweeningModel(TWeakPtr<FCurveEditor> InWeakCurveEditor);

	//~ Begin FTweenModel Interface
	virtual void SetScaleMode(TweeningUtilsEditor::ETweenScaleMode InMode) override;
	virtual void StartBlendOperation() override;
	virtual void StopBlendOperation() override;
	virtual void BlendValues(float InNormalizedValue) override;
	//~ End FTweenModel Interface

private:

	/** The curve editor of which the keys are tweened. */
	const TWeakPtr<FCurveEditor> CurveEditor;

	/** Tweens when there is a key selection. */
	TweeningUtilsEditor::TCurveTweenModel<BlendFunction> SelectionBasedModel;
	/** Injected model that tweens when there is no key selection. Can be nullptr. */
	const TSharedPtr<FTweenModel> NoKeySelectionModel;

	template<typename TCallback> requires std::is_invocable_v<TCallback, TweeningUtilsEditor::FTweenModel&>
	void AccessCurrentModel(TCallback&& InCallback);
};
}


namespace UE::CurveEditorTools
{
template <TweeningUtilsEditor::EBlendFunction BlendFunction>
TContextAwareTweeningModel<BlendFunction>::TContextAwareTweeningModel(TWeakPtr<FCurveEditor> InWeakCurveEditor)
	: CurveEditor(InWeakCurveEditor)
	, SelectionBasedModel(InWeakCurveEditor)
	, NoKeySelectionModel(MakeShared<TKeyInsertingTweenModel<BlendFunction>>(InWeakCurveEditor))
{}

template <TweeningUtilsEditor::EBlendFunction BlendFunction>
void TContextAwareTweeningModel<BlendFunction>::SetScaleMode(TweeningUtilsEditor::ETweenScaleMode InMode)
{
	Super::SetScaleMode(InMode);
	SelectionBasedModel.SetScaleMode(InMode);
	if (NoKeySelectionModel)
	{
		NoKeySelectionModel->SetScaleMode(InMode);
	}
}

template <TweeningUtilsEditor::EBlendFunction BlendFunction>
void TContextAwareTweeningModel<BlendFunction>::StartBlendOperation()
{
	AccessCurrentModel([this](FTweenModel& Model)
	{
		Model.StartBlendOperation();
	});
}

template <TweeningUtilsEditor::EBlendFunction BlendFunction>
void TContextAwareTweeningModel<BlendFunction>::StopBlendOperation()
{
	AccessCurrentModel([](FTweenModel& Model)
	{
		Model.StopBlendOperation();
	});
}

template <TweeningUtilsEditor::EBlendFunction BlendFunction>
void TContextAwareTweeningModel<BlendFunction>::BlendValues(float InNormalizedValue)
{
	AccessCurrentModel([InNormalizedValue](FTweenModel& Model)
	{
		Model.BlendValues(InNormalizedValue);
	});
}

template <TweeningUtilsEditor::EBlendFunction BlendFunction>
template <typename TCallback> requires std::is_invocable_v<TCallback, TweeningUtilsEditor::FTweenModel&>
void TContextAwareTweeningModel<BlendFunction>::AccessCurrentModel(TCallback&& InCallback)
{
	if (SelectionBasedModel.HasAnythingToBlend())
	{
		InCallback(SelectionBasedModel);
	}
	else if (NoKeySelectionModel)
	{
		InCallback(*NoKeySelectionModel.Get());
	}
}
}