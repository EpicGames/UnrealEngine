// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorTweenModels.h"

#include "ContextAwareTweeningModel.h"
#include "Math/Models/CurveTimeOffsetTweenModel.h"
#include "Math/Models/EditorTweenModel.h"

namespace UE::CurveEditorTools
{
namespace TweenDetail
{
static TArray<TweeningUtilsEditor::FTweenModelUIEntry> MakeTweenModels(TWeakPtr<FCurveEditor> InCurveEditor)
{
	using namespace TweeningUtilsEditor;
	
	TArray<FTweenModelUIEntry> Result;
	const auto AddCurveTweenable = [&InCurveEditor, &Result]<EBlendFunction TBlendFunction>
	{
		using FModel = TEditorTweenModel<TContextAwareTweeningModel<TBlendFunction>>;
		TUniquePtr<FModel> TweenModel = MakeUnique<FModel>(
			InCurveEditor,			// TTransactionalTweenModelProxy
			InCurveEditor,			// TTangentFlatteningTweenProxy
			InCurveEditor			// FContextAwareTweeningModel
			);
		Result.Emplace(MoveTemp(TweenModel), FTweenModelDisplayInfo(TBlendFunction));
	};
	static_assert(static_cast<int32>(EBlendFunction::Num) == 7, "You probably want to add the new tween function here.");
	
	// Could use TweeningUtilsEditor::ForEachCurveTweenable, but the functions are supposed to be listed in this specific order.
	AddCurveTweenable.template operator()<EBlendFunction::BlendNeighbor>();
	AddCurveTweenable.template operator()<EBlendFunction::PushPull>();
	AddCurveTweenable.template operator()<EBlendFunction::BlendEase>();
	AddCurveTweenable.template operator()<EBlendFunction::BlendRelative>();
	Result.Emplace(
		MakeUnique<TEditorTweenModel<FCurveTimeOffsetTweenModel>>(
			InCurveEditor,			// TTransactionalTweenModelProxy
			InCurveEditor,		// TTangentFlatteningTweenProxy
			InCurveEditor		// FCurveTimeOffsetTweenModel
			),
		FTweenModelDisplayInfo(EBlendFunction::TimeOffset)
	);
	AddCurveTweenable.template operator()<EBlendFunction::SmoothRough>();
	AddCurveTweenable.template operator()<EBlendFunction::ControlsToTween>();

	return Result;
}
}
	
FCurveEditorTweenModels::FCurveEditorTweenModels(TWeakPtr<FCurveEditor> InCurveEditor)
	: FTweenModelArray(TweenDetail::MakeTweenModels(InCurveEditor))
{}
}
