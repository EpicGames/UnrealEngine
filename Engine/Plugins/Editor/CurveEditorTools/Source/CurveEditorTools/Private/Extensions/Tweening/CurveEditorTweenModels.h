// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Abstraction/TweenModelArray.h"

class FCurveEditor;

namespace UE::CurveEditorTools
{
/** Knows of the tween models used in curve editor. */
class FCurveEditorTweenModels : public TweeningUtilsEditor::FTweenModelArray
{
public:

	explicit FCurveEditorTweenModels(const TWeakPtr<FCurveEditor> InCurveEditor);
};
}

