// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Math/Models/CurveTweenModel.h"

class FCurveEditor;

namespace UE::CurveEditorTools
{
/** Inserts keys of visible curves into the current scrubber position. */
template<TweeningUtilsEditor::EBlendFunction BlendFunction>
class TKeyInsertingTweenModel : public TweeningUtilsEditor::TCurveTweenModel<BlendFunction>
{
	using Super = TweeningUtilsEditor::TCurveTweenModel<BlendFunction>;
public:

	explicit TKeyInsertingTweenModel(TWeakPtr<FCurveEditor> InWeakCurveEditor) : Super(MoveTemp(InWeakCurveEditor)) {}

	//~ Begin FTweenModel
	virtual void StartBlendOperation() override;
	//~ Begin FTweenModel
};
}

namespace UE::CurveEditorTools
{
template <TweeningUtilsEditor::EBlendFunction BlendFunction>
void TKeyInsertingTweenModel<BlendFunction>::StartBlendOperation()
{
	const TSharedPtr<FCurveEditor> CurveEditor = this->WeakCurveEditor.Pin();
	if (!ensure(CurveEditor))
	{
		return;
	}

	// Not all curve editors have a time slider controller
	const TSharedPtr<ITimeSliderController> TimeSliderController = CurveEditor->GetTimeSliderController();
	if (!TimeSliderController)
	{
		return;
	}
	
	const double ScrubTime = TimeSliderController->GetTickResolution().AsSeconds(TimeSliderController->GetScrubPosition());
	for (const TTuple<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState>& Pair : CurveEditor->GetTreeSelection())
	{
		if (Pair.Value == ECurveEditorTreeSelectionState::None)
		{
			continue;
		}
		
		const FCurveEditorTreeItem& TreeItem = CurveEditor->GetTreeItem(Pair.Key);
		for (const FCurveModelID& CurveModelID : TreeItem.GetCurves())
		{
			FCurveModel* CurveModel = CurveEditor->FindCurve(CurveModelID);
			if (!CurveModel)
			{
				return;
			}

			double ValueAtScrubTime;
			if (!CurveModel->Evaluate(ScrubTime, ValueAtScrubTime))
			{
				continue;
			}

			// Try to insert the key with similar interpolation modes as its neighbours
			FKeyAttributes KeyAttributes;
			const TPair<ERichCurveInterpMode, ERichCurveTangentMode> Modes = CurveModel->GetInterpolationMode(ScrubTime, RCIM_Cubic, RCTM_SmartAuto);
			KeyAttributes.SetInterpMode(Modes.Key);
			KeyAttributes.SetTangentMode(Modes.Value);

			const TOptional<FKeyHandle> KeyHandle = CurveModel->AddKey({ ScrubTime, ValueAtScrubTime }, KeyAttributes);
			if (!KeyHandle)
			{
				continue;
			}

			// This will make Blend function blend the key we just inserted
			this->ContiguousKeySelection.Append(*CurveEditor, CurveModelID, { *KeyHandle });
		}
	}
}
}