// Copyright Epic Games, Inc. All Rights Reserved.

#include "TweenEditorExtension.h"

#include "ContextAwareTweeningModel.h"
#include "CurveEditor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Misc/Optional.h"
#include "Misc/ResizeParamUtils.h"
#include "Widgets/MVC/TweenToolbarController.h"

namespace UE::CurveEditorTools
{
FTweenEditorExtension::FTweenEditorExtension(TWeakPtr<FCurveEditor> InCurveEditor)
	: WeakCurveEditor(InCurveEditor)
	, TweenModelContainer(MakeShared<FCurveEditorTweenModels>(InCurveEditor))
{}

void FTweenEditorExtension::BindCommands(TSharedRef<FUICommandList> InCommandList)
{
	InitControllers(InCommandList);
}

TSharedPtr<FExtender> FTweenEditorExtension::MakeToolbarExtender(const TSharedRef<FUICommandList>& InCommandList)
{
	using namespace TweeningUtilsEditor;
	TSharedRef<FExtender> Extender = MakeShared<FExtender>();
	
	Extender->AddToolBarExtension("Adjustment", EExtensionHook::After, InCommandList, FToolBarExtensionDelegate::CreateLambda(
		[this, WeakCommandList = InCommandList.ToWeakPtr()](FToolBarBuilder& ToolbarBuilder)
		{
			const TSharedPtr<FUICommandList> CommandListPin = WeakCommandList.Pin();
			if (ensure(CommandListPin) && InitControllers(CommandListPin.ToSharedRef()))
			{
				// Visually move the slider widget when the user uses the U+LMB button command to indirectly move the mouse
				TAttribute<TOptional<float>> OverrideSliderPosition =
					TAttribute<TOptional<float>>::CreateLambda([this]()
					{
						return TweenControllers->GetCurrentSliderPosition();
					});
				
				ToolbarBuilder.BeginStyleOverride(TEXT("CurveEditorTweenToolbar")); // Forces min sizes for combo button
				ToolbarBuilder.BeginSection("Tween");
				TweenControllers->ToolbarController.AddToToolbar(ToolbarBuilder,
				{
					.OverrideSliderPositionAttr = MoveTemp(OverrideSliderPosition),
					.FunctionSelectResizeParams = CurveEditor::MakeResizeParams(TEXT("Tween.FunctionSelect")),
					.SliderResizeParams			= CurveEditor::MakeResizeParams(TEXT("Tween.Slider")),
					.OvershootResizeParams		= CurveEditor::MakeResizeParams(TEXT("Tween.Overshoot")) 
				});
				ToolbarBuilder.EndSection();
				ToolbarBuilder.EndStyleOverride();
			}
		}));
	
	return Extender;
}

bool FTweenEditorExtension::InitControllers(TSharedRef<FUICommandList> InCommandList)
{
	if (TweenControllers)
	{
		return true;
	}

	if (const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin(); ensure(CurveEditorPin))
	{
		TweenControllers.Emplace(InCommandList, TweenModelContainer, TEXT("CurveEditorTween"));
		return true;
	}
	return false;
}
}
