// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditorTweenModels.h"
#include "ICurveEditorExtension.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Widgets/MVC/TweenControllers.h"

class FCurveEditor;
class FExtender;
namespace UE::TweeningUtilsEditor { class FTweenModelArray; }

namespace UE::CurveEditorTools
{
/** Manages the tweening tool in the curve editor. */
class FTweenEditorExtension : public ICurveEditorExtension, public FNoncopyable
{
public:

	explicit FTweenEditorExtension(TWeakPtr<FCurveEditor> InCurveEditor);

	//~ Begin ICurveEditorExtension Interface
	virtual void BindCommands(TSharedRef<FUICommandList> InCommandList) override;
	virtual TSharedPtr<FExtender> MakeToolbarExtender(const TSharedRef<FUICommandList>& InCommandList) override;
	//~ End ICurveEditorExtension Interface

private:

	/** The editor that owns us. */
	const TWeakPtr<FCurveEditor> WeakCurveEditor;

	/** Holds the functions that ToolbarController shows: just the default, built-in ones.*/
	const TSharedRef<FCurveEditorTweenModels> TweenModelContainer;

	/** Created by InitControllers. */
	TOptional<TweeningUtilsEditor::FTweenControllers> TweenControllers;

	/** Makes sure that ToolbarController is created. */
	bool InitControllers(TSharedRef<FUICommandList> InCommandList);
};
}

