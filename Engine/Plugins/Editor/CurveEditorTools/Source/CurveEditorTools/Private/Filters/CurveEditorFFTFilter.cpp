// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorFFTFilter.h"
#include "DSP/PassiveFilter.h"
#include "Filters/CurveEditorBakeFilter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveEditorFFTFilter)

UCurveEditorFFTFilter::UCurveEditorFFTFilter()
{
	CutoffFrequency = 0.8f;
	Order = 4;
	Type = ECurveEditorFFTFilterType::Lowpass;
	Response = ECurveEditorFFTFilterClass::Chebyshev;
}

void UCurveEditorFFTFilter::ApplyFilter_Impl(TSharedRef<FCurveEditor> InCurveEditor, const TMap<FCurveModelID, FKeyHandleSet>& InKeysToOperateOn, TMap<FCurveModelID, FKeyHandleSet>& OutKeysToSelect)
{
	const FFrameRate DisplayRate = InCurveEditor->GetTimeSliderController() ? InCurveEditor->GetTimeSliderController()->GetDisplayRate() : FFrameRate();
	const FFrameRate TickResolution = InCurveEditor->GetTimeSliderController() ? InCurveEditor->GetTimeSliderController()->GetTickResolution() : FFrameRate();

	// FFT works on one curve at a time and expects the keys to have an even spacing between them.
	// We re-use the Bake filter to bake the supplied curve, but before we do this we need to cache
	// the original key times so that we can evaluate the filtered curve and update the original curve
	// with the smoothed out values.
	// Now that we've backed up the original time and values of the keys, we need to bake it down to have even spacing. This is important for FFT.
	UCurveEditorBakeFilter* BakeFilter = UCurveEditorBakeFilter::StaticClass()->GetDefaultObject<UCurveEditorBakeFilter>();
	BakeFilter->InitializeFilter(InCurveEditor);
	
	// Since we're baking under the hood, we need to cache their bake interval, override it to a reasonable number, and restore it at the end.
	FFrameNumber OriginalBakeInterval = BakeFilter->BakeInterval;
	float OriginalBakeIntervalInSeconds = BakeFilter->BakeIntervalInSeconds;

	TArray<FKeyHandle> OriginalKeyHandles;
	TArray<FKeyPosition> SelectedKeyPositions;
	
	for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : InKeysToOperateOn)
	{
		FCurveModel* CurveModel = InCurveEditor->FindCurve(Pair.Key);
		check(CurveModel);

		OriginalKeyHandles.Reset(Pair.Value.Num());
		OriginalKeyHandles.Append(Pair.Value.AsArray().GetData(), Pair.Value.Num());

		// We get all the keys within the bake range on the curve, so we can store a copy of their positions.
		SelectedKeyPositions.SetNumUninitialized(OriginalKeyHandles.Num());
		CurveModel->GetKeyPositions(OriginalKeyHandles, SelectedKeyPositions);
		
		// Find the hull of the range of the selected keys
		double MinKey = TNumericLimits<double>::Max(), MaxKey = TNumericLimits<double>::Lowest();
		for (FKeyPosition Key : SelectedKeyPositions)
		{
			MinKey = FMath::Min(Key.InputValue, MinKey);
			MaxKey = FMath::Max(Key.InputValue, MaxKey);
		}

		// Get all keys that exist between the time range
		OriginalKeyHandles.Reset();
		CurveModel->GetKeys(MinKey, MaxKey, TNumericLimits<double>::Lowest(), TNumericLimits<double>::Max(), OriginalKeyHandles);
		
		// Need at least three keys to filter
		if (OriginalKeyHandles.Num() < 3)
		{
			continue;
		}
		//Set interval range divded by twice the number of keys.

		double IntervalAsSeconds = (MaxKey - MinKey) / (2 * OriginalKeyHandles.Num());
		BakeFilter->BakeIntervalInSeconds = IntervalAsSeconds;
		BakeFilter->BakeInterval = (IntervalAsSeconds * TickResolution).FloorToFrame();

		// This stores the position of all of the original keys. Once we've filtered the curve we will need to sample it at these positions.
		TArray<FKeyPosition> OriginalKeyPositions;
		TArray<FKeyAttributes> OriginalKeyAttributes;
		OriginalKeyPositions.SetNumUninitialized(OriginalKeyHandles.Num());
		OriginalKeyAttributes.SetNumUninitialized(OriginalKeyHandles.Num());
		CurveModel->GetKeyPositions(OriginalKeyHandles, OriginalKeyPositions);
		CurveModel->GetKeyAttributes(OriginalKeyHandles, OriginalKeyAttributes);

		// Apply the Bake filter to the same segment of the curve as this. This invalidates the original KeyHandles, because it creates and destroys keys.
		TArray<FKeyHandle> BakedKeyHandles;
		BakeFilter->ApplyFilter(InCurveEditor, Pair.Key, MakeArrayView(OriginalKeyHandles.GetData(), OriginalKeyHandles.Num()), BakedKeyHandles);
		// Get the positions from the baked curve to feed into the FFT samples.
		TArray<FKeyPosition> BakedKeyPositions;
		BakedKeyPositions.SetNumUninitialized(BakedKeyHandles.Num());	
		CurveModel->GetKeyPositions(BakedKeyHandles, BakedKeyPositions);

		//Bake Filter returns handles unsorted so need to sort. (Second key is last key but they may change so do it manually).
		TArray<TPair < FKeyHandle, FKeyPosition>> KeyAndPosition;
		KeyAndPosition.SetNumUninitialized(BakedKeyHandles.Num());
		int32 Index = 0;
		for (; Index < BakedKeyHandles.Num(); ++Index)
		{
			KeyAndPosition[Index] = TPair<FKeyHandle, FKeyPosition>(BakedKeyHandles[Index], BakedKeyPositions[Index]);
		}

		Algo::Sort(KeyAndPosition, [](const TPair < FKeyHandle, FKeyPosition> LHS, const TPair < FKeyHandle, FKeyPosition> RHS)
		{
			return LHS.Value.InputValue < RHS.Value.InputValue;
		});

		for (Index = 0; Index < KeyAndPosition.Num(); ++Index)
		{
			BakedKeyHandles[Index] = KeyAndPosition[Index].Key;
			BakedKeyPositions[Index] = KeyAndPosition[Index].Value;
		}

		//now it's sorted we can just use them.

		TArray<float> FFTSamples;
		FFTSamples.Reserve(BakedKeyPositions.Num());

		double MinValue = TNumericLimits<double>::Max(), MaxValue = TNumericLimits<double>::Lowest();
		for (const FKeyPosition& Position : BakedKeyPositions)
		{
			// We have to cast to float... probably okay, most of the things you're animating are floats so their values would be in float ranges.
			FFTSamples.Add((float)Position.OutputValue);
			MinValue = FMath::Min(Position.OutputValue, MinValue);
			MaxValue = FMath::Max(Position.OutputValue, MaxValue);
		}

		// We have to offset our samples to center them around the 0 value as the filtering code expects that.
		double ValueOffset = MinValue + ((MaxValue - MinValue) / 2.0);
		for (float& Sample : FFTSamples)
		{
			Sample -= ValueOffset;
		}

		// Now that the FFT samples have been filled out, we can pass this to the filtering code.
		Audio::FPassiveFilterParams Params;
		Params.Type = (Type == ECurveEditorFFTFilterType::Highpass) ? Audio::FPassiveFilterParams::EType::Highpass : Audio::FPassiveFilterParams::EType::Lowpass; // Enum Conversion
		Params.Class = (Response == ECurveEditorFFTFilterClass::Butterworth) ? Audio::FPassiveFilterParams::EClass::Butterworth : Audio::FPassiveFilterParams::EClass::Chebyshev; // Enum Conversion
		Params.NormalizedCutoffFrequency = CutoffFrequency;
		Params.Order = Order;
		Params.bScaleByOffset = false;
		// Our CurveModel now stores dense curve data. FFT filtering works only on values, and requires a POT sized array. Instead of duplicating the 
		// logic to round up to a POT we use the version of audio filtering which does it for us by adding additional zero samples.
		Audio::Filter(FFTSamples, Params);

		// Now we can apply the filtered data back to our curve, which will allow us to sample it. We only loop up to the number
		// of handles, as FFT has padded our array (with zeros) up to the next POT.
		for (int32 KeyHandleIndex = 0; KeyHandleIndex < BakedKeyHandles.Num(); KeyHandleIndex++)
		{
			if (!FMath::IsNaN(FFTSamples[KeyHandleIndex]))
			{
				BakedKeyPositions[KeyHandleIndex].OutputValue = FFTSamples[KeyHandleIndex] + ValueOffset;
			}
		}

		// We re-assign the filtered samples back to our baked curve so that we can then sample it at the original times.
		CurveModel->SetKeyPositions(BakedKeyHandles, BakedKeyPositions);

		// Now we sample the smoothed curve at the original locations to find out what the new values are...
		for (FKeyPosition& KeyPosition : OriginalKeyPositions)
		{
			double Sample;
			CurveModel->Evaluate(KeyPosition.InputValue, Sample);

			// Override the output value now, no need to track a separate array.
			KeyPosition.OutputValue = Sample;
		}

		// Our curve model is still full of baked data. We do this as a second loop otherwise as we added keys, the sampling evaluation would be incorrect.
		CurveModel->RemoveKeys(BakedKeyHandles, 0.0);

		FKeyHandleSet& SelectedHandleSet = OutKeysToSelect.Add(Pair.Key);
		for(int32 KeyIndex = 0; KeyIndex < OriginalKeyPositions.Num(); KeyIndex++)
		{
			TOptional<FKeyHandle> Handle = CurveModel->AddKey(OriginalKeyPositions[KeyIndex], OriginalKeyAttributes[KeyIndex]);
			if (Handle.IsSet())
			{
				SelectedHandleSet.Add(Handle.GetValue(), ECurvePointType::Key);
			}
		}
	}

	// Restore their Bake Filter settings.
	BakeFilter->BakeInterval = OriginalBakeInterval;
	BakeFilter->BakeIntervalInSeconds = OriginalBakeIntervalInSeconds;
}

