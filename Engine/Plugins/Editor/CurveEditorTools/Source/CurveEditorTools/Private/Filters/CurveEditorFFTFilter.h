// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/CurveEditorFilterBase.h"
#include "CurveEditorFFTFilter.generated.h"

struct FCurveModelID;
struct FKeyHandleSet;


UENUM(BlueprintType)
enum class ECurveEditorFFTFilterClass : uint8
{
	Butterworth,
	Chebyshev
};

UENUM(BlueprintType)
enum class ECurveEditorFFTFilterType : uint8
{
	Lowpass,
	Highpass
};

UCLASS(DisplayName="Fourier Transform (FFT)")
class UCurveEditorFFTFilter : public UCurveEditorFilterBase
{
	GENERATED_BODY()
protected:
	void ApplyFilter_Impl(TSharedRef<FCurveEditor> InCurveEditor, const TMap<FCurveModelID, FKeyHandleSet>& InKeysToOperateOn, TMap<FCurveModelID, FKeyHandleSet>& OutKeysToSelect) override;
public:
	UCurveEditorFFTFilter();

	/** Normalized between 0-1. In a low pass filter, the lower the value is the smoother the output. In a high pass filter the higher the value the smoother the output.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = 0, UIMax = 1), Category="Settings")
	float CutoffFrequency;

	/** Which frequencies are allowed through. For example, low-pass will let low frequency through and remove high frequency noise. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	ECurveEditorFFTFilterType Type;
	
	/** Which FFT filter implementation to use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	ECurveEditorFFTFilterClass Response;

	/** The number of samples used to filter in the time domain. It maps how steep the roll off is for the filter. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta=(UIMin=1, UIMax=8), Category = "Settings")
	int32 Order;
};
