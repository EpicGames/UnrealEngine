// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurvePointSnapper.h"

#include "CurveEditor.h"
#include "CurveEditorTypes.h"
#include "SCurveEditorView.h"
#include "Misc/Optional.h"

namespace UE::CurveEditorTools
{
TOptional<FCurvePointSnapper> FCurvePointSnapper::MakeSnapper(FCurveEditor& InCurveEditor, const FCurveModelID& InCurveModelId)
{
	if (InCurveEditor.GetSelection().GetAll().IsEmpty())
	{
		return {};
	}
	
	const SCurveEditorView* View = InCurveEditor.FindFirstInteractiveView(InCurveModelId);
	if (!View)
	{
		return {};
	}

	return FCurvePointSnapper(View->GetCurveSpace(InCurveModelId), InCurveEditor.GetCurveSnapMetrics(InCurveModelId));
}

FVector2D FCurvePointSnapper::SnapPoint(const FVector2D& InCurveSpacePoint, EKeyMoveMode InDragMode) const
{
	// LockedX & LockedY mean that the user is e.g. shift dragging: this means they're intentionally moving in only one direction.
	return FVector2D
	{
		// If moving only up & down, then it would be weird if the keys snapped in x direction.
		EnumHasAnyFlags(InDragMode, EKeyMoveMode::IgnoreX) ? InCurveSpacePoint.X : SnapMetrics.SnapInputSeconds(InCurveSpacePoint.X),
		// If moving only left & right, then it would be weird if the keys snapped in y direction.
		EnumHasAnyFlags(InDragMode, EKeyMoveMode::IgnoreY) ? InCurveSpacePoint.Y : SnapMetrics.SnapOutput(InCurveSpacePoint.Y)
	};
}

FKeyPosition FCurvePointSnapper::SnapKey(const FKeyPosition& InKeyPosition, EKeyMoveMode InDragMode) const
{
	const FVector2D Point = SnapPoint({ InKeyPosition.InputValue, InKeyPosition.OutputValue }, InDragMode);
	return { Point.X, Point.Y };
}
}
