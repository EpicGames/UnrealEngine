// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditorScreenSpace.h"
#include "CurveEditorSnapMetrics.h"
#include "Tools/Lattice/Drag/EKeyMoveMode.h"

struct FCurveModelID;
struct FKeyPosition;
class FCurveEditor;
template<typename OptionalType> struct TOptional;

namespace UE::CurveEditorTools
{
/** Util that you can snap points according to a curve editor's snapping settings. */
class FCurvePointSnapper
{
public:

	/** @return Unset if there's no curve view from which to get the FCurveEditorScreenSpace transform. */
	static TOptional<FCurvePointSnapper> MakeSnapper(FCurveEditor& InCurveEditor, const FCurveModelID& InCurveModelId);

	/** Snaps the point as per the snapping settings */
	FVector2D SnapPoint(const FVector2D& InCurveSpacePoint, EKeyMoveMode InDragMode = EKeyMoveMode::Unrestricted) const;

	/** Snaps the key as per the snapping settings */
	FKeyPosition SnapKey(const FKeyPosition& InKeyPosition, EKeyMoveMode InDragMode = EKeyMoveMode::Unrestricted) const;

private:

	explicit FCurvePointSnapper(const FCurveEditorScreenSpace& InCurveSpace, const FCurveSnapMetrics& InMetrics)
		: CurveSpace(InCurveSpace), SnapMetrics(InMetrics)
	{}

	const FCurveEditorScreenSpace CurveSpace;
	const FCurveSnapMetrics SnapMetrics;
};
}

