// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveViewRebuildListener.h"

#include "CurveEditor.h"
#include "SCurveEditorPanel.h"

namespace UE::CurveEditorTools
{
FCurveViewRebuildListener::FCurveViewRebuildListener(const TSharedRef<FCurveEditor>& InCurveEditor)
	: WeakCurveEditor(InCurveEditor)
{
	const TSharedPtr<SCurveEditorPanel> Panel = InCurveEditor->GetPanel();
	if (ensure(Panel))
	{
		Panel->OnPostRebuildCurveViews().AddRaw(this, &FCurveViewRebuildListener::BroadcastOnCurveModified);
	}
}

FCurveViewRebuildListener::~FCurveViewRebuildListener()
{
	const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin();
	const TSharedPtr<SCurveEditorPanel> Panel = CurveEditorPin ? CurveEditorPin->GetPanel() : nullptr;
	if (Panel)
	{
		Panel->OnPostRebuildCurveViews().RemoveAll(this);
	}
}
}
