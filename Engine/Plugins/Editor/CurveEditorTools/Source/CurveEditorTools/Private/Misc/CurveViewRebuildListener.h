// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class FCurveEditor;
struct FCurveModelID;

namespace UE::CurveEditorTools
{
/**
 * Invokes OnCurveViewRebuilt when the curve editor's SCurveEditorPanel::OnPostRebuildCurveViews is invoked.
 * This handles clean-up, etc. so the using code does not need to worry about it.
 */
class FCurveViewRebuildListener : public FNoncopyable
{
public:

	explicit FCurveViewRebuildListener(const TSharedRef<FCurveEditor>& InCurveEditor);
	~FCurveViewRebuildListener();

	FSimpleMulticastDelegate& OnCurveViewRebuilt() { return OnCurveModifiedDelegate; }

private:

	/** Used to unsubscribe on destruction. */
	TWeakPtr<FCurveEditor> WeakCurveEditor;

	/** Invokes when any of the listeners is changed. */
	FSimpleMulticastDelegate OnCurveModifiedDelegate;

	void BroadcastOnCurveModified() const { OnCurveModifiedDelegate.Broadcast(); }
};
}