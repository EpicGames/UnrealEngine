// Copyright Epic Games, Inc. All Rights Reserved.

#include "VectorMathUtils.h"

#include "Math/Vector2D.h"

namespace UE::CurveEditorTools
{
static double CrossProductSign(const FVector2D& Point, const FVector2D& Vertex1, const FVector2D& Vertex2)
{
	const FVector2D Vertex1ToPoint = Point - Vertex1;
	const FVector2D Vertex1ToVertex2 = Vertex2 - Vertex1;
	return FVector2D::CrossProduct(Vertex1ToPoint, Vertex1ToVertex2);
}

bool IsPointInTriangle(const FVector2D& Point, const FVector2D& Vertex1, const FVector2D& Vertex2, const FVector2D& Vertex3)
{
	// The idea is to compute the 2D cross product between the point and each edge.
	// Point x edge < 0 means point is to the right
	// Point x edge > 0 means point is to the left
	const double Sign_1_2 = CrossProductSign(Point, Vertex1, Vertex2);
	const double Sign_2_3 = CrossProductSign(Point, Vertex2, Vertex3);
	const double Sign_3_1 = CrossProductSign(Point, Vertex3, Vertex1);

	// If the point is on the same side of each edge, that means it is inside the triangle.
	const bool bHasPositive = Sign_1_2 > 0.0 || Sign_2_3 > 0.0 || Sign_3_1 > 0.0;
	const bool bHasNegative = Sign_1_2 < 0.0 || Sign_2_3 < 0.0 || Sign_3_1 < 0.0;
	return !(bHasPositive && bHasNegative);
}

void InsetQuadBy(FVector2D& A, FVector2D& B, FVector2D& C, FVector2D& D, float InsetAmount)
{
	// Calculate edge vectors and normals
	const FVector2D AB = (B - A).GetSafeNormal(); 
	const FVector2D BC = (C - B).GetSafeNormal(); 
	const FVector2D CD = (D - C).GetSafeNormal(); 
	const FVector2D DA = (A - D).GetSafeNormal(); 

	// Calculate inward normals (perpendiculars)
	const FVector2D InwardAB = FVector2D(-AB.Y, AB.X); 
	const FVector2D InwardBC = FVector2D(-BC.Y, BC.X); 
	const FVector2D InwardCD = FVector2D(-CD.Y, CD.X); 
	const FVector2D InwardDA = FVector2D(-DA.Y, DA.X); 

	// Offset each vertex by the inward normals of adjacent edges
	A = A + -InsetAmount * (InwardDA + InwardAB).GetSafeNormal();
	B = B + -InsetAmount * (InwardAB + InwardBC).GetSafeNormal();
	C = C + -InsetAmount * (InwardBC + InwardCD).GetSafeNormal();
	D = D + -InsetAmount * (InwardCD + InwardDA).GetSafeNormal();
}

FTransform2d TransformRectBetweenSpaces(
	const FVector2D& InMinSource, const FVector2D& InMaxSource,
	const FVector2D& InMinTarget, const FVector2D& InMaxTarget
	)
{
	const FVector2D DeltaLattice = InMaxSource - InMinSource;
	const FVector2D DeltaCurveSpace = InMaxTarget - InMinTarget; 
	const FTransform2d ToCurveSpace = Concatenate(
		-InMinSource,									// Translate absolute to origin 
		FScale2d(FVector2D(1., 1.) / DeltaLattice),	// Normalize to [0,1] range
		FScale2d(DeltaCurveSpace),					// Rescale to curve space
		InMinTarget								// Translate to curve space offset
		);
	return ToCurveSpace;
}
	
FVector2D TransformAbsoluteToCurveSpace(const FTransform2d& InAbsToCurveSpace, const FVector2D& InPoint)
{
	double M00, M01, M10, M11;
	InAbsToCurveSpace.GetMatrix().GetMatrix(M00, M01, M10, M11);

	FVector2D CurveSpacePosition = InPoint - InAbsToCurveSpace.GetTranslation();
	CurveSpacePosition.X /= M00;
	CurveSpacePosition.Y /= M11;
	return CurveSpacePosition;
}
	
FVector2D TransformCurveSpaceToAbsolute(const FTransform2d& InAbsToCurveSpace, const FVector2D& InPoint)
{
	double M00, M01, M10, M11;
	InAbsToCurveSpace.GetMatrix().GetMatrix(M00, M01, M10, M11);
	return FVector2D(M00, M11) * InPoint + InAbsToCurveSpace.GetTranslation();
}
}
