// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "Math/TransformCalculus2D.h"

namespace UE::CurveEditorTools
{
/** @return Whether Point is in the triangle spanned by the input vertices. */
bool IsPointInTriangle(const FVector2D& Point, const FVector2D& Vertex1, const FVector2D& Vertex2, const FVector2D& Vertex3);
/** @return Makes a smaller quad in the same shape but with the edges moved inside by InsetAmount. */
void InsetQuadBy(FVector2D& A, FVector2D& B, FVector2D& C, FVector2D& D, float InsetAmount);

/**
 * Computes the transform for transforming a rectangle from source space to another target space.
 *
 * @param InMinSource The bottom left corner of the rectangle in absolute space
 * @param InMaxSource The top right corner of the rectangle in absolute space
 * @param InMinTarget The equivalent bottom left corner in curve space
 * @param InMaxTarget The equivalent top right corner in curve space
 */
FTransform2d TransformRectBetweenSpaces(
	const FVector2D& InMinSource, const FVector2D& InMaxSource,
	const FVector2D& InMinTarget, const FVector2D& InMaxTarget
	);

/**
 * Transforms a point from absolute key space to a SCurveEditorView's curve space.
 * 
 * Absolute key space are the literal FKeyPosition::InputValue and OutputValue.
 * Curve space are the axis values the SCurveEditorView displays the keys at (e.g. in Normalized the max value is displayed at curve space value 1.0).
 * 
 * @param InAbsToCurveSpace Result of SCurveEditorView::GetViewToCurveTransform
 * @param InPoint The point in absolute space to transform
 * @return The point in curve space
 */
FVector2D TransformAbsoluteToCurveSpace(const FTransform2d& InAbsToCurveSpace, const FVector2D& InPoint);
/**
 * Transforms a point from SCurveEditorView's curve space to absolute key space.
 * 
 * Absolute key space are the literal FKeyPosition::InputValue and OutputValue.
 * Curve space are the axis values the SCurveEditorView displays the keys at (e.g. in Normalized the max value is displayed at curve space value 1.0).
 * 
 * @param InAbsToCurveSpace Result of SCurveEditorView::GetViewToCurveTransform
 * @param InPoint The point in curve space to transform
 * @return The point in absolute space
 */
FVector2D TransformCurveSpaceToAbsolute(const FTransform2d& InAbsToCurveSpace, const FVector2D& InPoint);
}
