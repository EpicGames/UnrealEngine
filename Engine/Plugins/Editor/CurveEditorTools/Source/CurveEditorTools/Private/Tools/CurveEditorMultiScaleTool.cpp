// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorMultiScaleTool.h"

#include "CurveEditorToolCommands.h"
#include "CurveEditor.h"
#include "Framework/Commands/UICommandList.h"
#include "CurveModel.h"
#include "SCurveEditorView.h"
#include "SCurveEditorPanel.h"
#include "Algo/MaxElement.h"
#include "Misc/Mirror/TangentSelectionFlattener.h"
#include "Modification/Keys/CurvesSnapshotBuilder.h"
#include "Modification/Utils/ScopedCurveChange.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveEditorMultiScaleTool)

#define LOCTEXT_NAMESPACE "CurveEditorToolCommands"

namespace CurveEditorMultiScaleTool
{
	constexpr float SliderWidth = 20.f;
	constexpr float SliderLength = 30.f;
	constexpr float SliderPadding = 2.f;
	constexpr float HighlightAlpha = 0.15f;
	constexpr float PivotRadius = 16.f;
}

EMultiScaleAnchorFlags FCurveEditorMultiScaleWidget::GetAnchorFlagsForMousePosition(const FGeometry& InWidgetGeometry, float InXDelta, float InYDelta, const FVector2D& InMouseScreenPosition) const 
{
	EMultiScaleAnchorFlags OutFlags = EMultiScaleAnchorFlags::None;

	FGeometry XSliderGeometry, YSliderGeometry, XSidebarGeometry, YSidebarGeometry;
	GetXSliderGeometry(InWidgetGeometry, InXDelta, XSliderGeometry);
	GetYSliderGeometry(InWidgetGeometry, InYDelta, YSliderGeometry);

	if (XSliderGeometry.IsUnderLocation(InMouseScreenPosition))
	{
		OutFlags |= EMultiScaleAnchorFlags::XSlider;
	}
	if (YSliderGeometry.IsUnderLocation(InMouseScreenPosition))
	{
		OutFlags |= EMultiScaleAnchorFlags::YSlider;
	}

	return OutFlags;
}

void FCurveEditorMultiScaleWidget::GetXSidebarGeometry(const FGeometry& InWidgetGeometry, const FGeometry& InViewGeometry, bool bInSelected, FGeometry& OutSidebar) const
{
	const float ViewExtent = InViewGeometry.GetLocalSize().X + 10.f;
	const FVector2D SidebarSize = FVector2D(bInSelected ? ViewExtent : InWidgetGeometry.GetLocalSize().X, CurveEditorMultiScaleTool::SliderWidth + 2.f * CurveEditorMultiScaleTool::SliderPadding);
	FVector2D SidebarOffset = FVector2D(bInSelected ? -Position.X - 5.f : (InWidgetGeometry.GetLocalSize().X * 0.5f) - (SidebarSize.X * 0.5f), InWidgetGeometry.GetLocalSize().Y + CurveEditorMultiScaleTool::SliderWidth - CurveEditorMultiScaleTool::SliderPadding);

	OutSidebar = InWidgetGeometry.MakeChild(SidebarSize, FSlateLayoutTransform(SidebarOffset));
}

void FCurveEditorMultiScaleWidget::GetYSidebarGeometry(const FGeometry& InWidgetGeometry, const FGeometry& InViewGeometry, bool bInSelected, FGeometry& OutSidebar) const
{
	const float ViewExtent = InViewGeometry.GetLocalSize().Y + 10.f;
	const FVector2D SidebarSize = FVector2D(CurveEditorMultiScaleTool::SliderWidth + 2.f * CurveEditorMultiScaleTool::SliderPadding, bInSelected ? ViewExtent : InWidgetGeometry.GetLocalSize().Y);
	const FVector2D SidebarOffset = FVector2D(InWidgetGeometry.GetLocalSize().X + CurveEditorMultiScaleTool::SliderWidth - CurveEditorMultiScaleTool::SliderPadding, bInSelected ? -Position.Y - 5.f : (InWidgetGeometry.GetLocalSize().Y * 0.5f) - (SidebarSize.Y * 0.5f));

	OutSidebar = InWidgetGeometry.MakeChild(SidebarSize, FSlateLayoutTransform(SidebarOffset));
}

void FCurveEditorMultiScaleWidget::GetXSliderGeometry(const FGeometry& InWidgetGeometry, float Delta, FGeometry& OutSlider) const
{
	const FVector2D SliderSize = FVector2D(CurveEditorMultiScaleTool::SliderLength, CurveEditorMultiScaleTool::SliderWidth);
	const FVector2D SliderOffset = FVector2D(ParentSpaceDragBegin.X - Position.X + Delta - SliderSize.X * .5f, InWidgetGeometry.GetLocalSize().Y + CurveEditorMultiScaleTool::SliderWidth);

	OutSlider = InWidgetGeometry.MakeChild(SliderSize, FSlateLayoutTransform(SliderOffset));
}

void FCurveEditorMultiScaleWidget::GetYSliderGeometry(const FGeometry& InWidgetGeometry, float Delta, FGeometry& OutSlider) const
{
	const FVector2D SliderSize = FVector2D(CurveEditorMultiScaleTool::SliderWidth, CurveEditorMultiScaleTool::SliderLength);
	const FVector2D SliderOffset = FVector2D(InWidgetGeometry.GetLocalSize().X + CurveEditorMultiScaleTool::SliderWidth, ParentSpaceDragBegin.Y - Position.Y + Delta - SliderSize.Y * .5f);

	OutSlider = InWidgetGeometry.MakeChild(SliderSize, FSlateLayoutTransform(SliderOffset));
}

void FCurveEditorMultiScaleTool::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 PaintOnLayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!MultiScaleWidget.Visible || !CurveEditor.IsValid() || (CurveEditor.IsValid() && CurveEditor->GetSelection().Count() <= 1))
	{
		return;
	}

	FGeometry BoundsGeometry = MultiScaleWidget.MakeGeometry(AllottedGeometry);

	const bool bXSliderHovered = (MultiScaleWidget.SelectedAnchorFlags & EMultiScaleAnchorFlags::XSlider) != EMultiScaleAnchorFlags::None;
	const bool bYSliderHovered = (MultiScaleWidget.SelectedAnchorFlags & EMultiScaleAnchorFlags::YSlider) != EMultiScaleAnchorFlags::None;

	// Draw sliders
	{
		FLinearColor XSliderHighlightColor = bXSliderHovered ?
			FLinearColor::White.CopyWithNewOpacity(CurveEditorMultiScaleTool::HighlightAlpha) : FLinearColor::Transparent;
		FLinearColor YSliderHighlightColor = bYSliderHovered ?
			FLinearColor::White.CopyWithNewOpacity(CurveEditorMultiScaleTool::HighlightAlpha) : FLinearColor::Transparent;
		FGeometry XSliderGeometry, YSliderGeometry;
		MultiScaleWidget.GetXSliderGeometry(BoundsGeometry, DragDelta.Get(FVector2D::ZeroVector).X, XSliderGeometry);
		MultiScaleWidget.GetYSliderGeometry(BoundsGeometry, DragDelta.Get(FVector2D::ZeroVector).Y, YSliderGeometry);

		FSlateDrawElement::MakeBox(OutDrawElements, PaintOnLayerId, XSliderGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("WhiteBrush")), ESlateDrawEffect::None, XSliderHighlightColor);
		FSlateDrawElement::MakeBox(OutDrawElements, PaintOnLayerId, XSliderGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("MarqueeSelection")));

		FSlateDrawElement::MakeBox(OutDrawElements, PaintOnLayerId, YSliderGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("WhiteBrush")), ESlateDrawEffect::None, YSliderHighlightColor);
		FSlateDrawElement::MakeBox(OutDrawElements, PaintOnLayerId, YSliderGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("MarqueeSelection")));
	}

	// Draw Sidebars
	{
		FGeometry XSidebarGeometry, YSidebarGeometry;
		MultiScaleWidget.GetXSidebarGeometry(BoundsGeometry, AllottedGeometry, bXSliderHovered, XSidebarGeometry);
		MultiScaleWidget.GetYSidebarGeometry(BoundsGeometry, AllottedGeometry, bYSliderHovered, YSidebarGeometry);
		FSlateDrawElement::MakeBox(OutDrawElements, PaintOnLayerId, XSidebarGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("MarqueeSelection")));
		FSlateDrawElement::MakeBox(OutDrawElements, PaintOnLayerId, YSidebarGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("MarqueeSelection")));
	}

	// Draw pivots
	if (DelayedDrag && DelayedDrag->IsDragging())
	{
		check(KeysByCurve.Num() == CurveEditor->GetSelection().GetAll().Num());
	}

	int32 KeysByCurveIdx = 0;
	for (TPair<FCurveModelID, FKeyHandleSet> Pair : CurveEditor->GetSelection().GetAll())
	{
		FCurveModelID CurveID = DelayedDrag.IsSet() && DelayedDrag->IsDragging() ? KeysByCurve[KeysByCurveIdx].CurveID : Pair.Key;
		FCurveModel*  Curve = CurveEditor->FindCurve(CurveID);

		if (ensureAlways(Curve))
		{
			const SCurveEditorView* View = CurveEditor->FindFirstInteractiveView(CurveID);
			if (!View)
			{
				continue;
			}

			TArrayView<const FKeyHandle> Handles = Pair.Value.AsArray();

			TArray<FKeyPosition> KeyPositions;
			KeyPositions.SetNumZeroed(Handles.Num());
			Curve->GetKeyPositions(Handles, KeyPositions);
			
			const FVector2D Pivot = DelayedDrag.IsSet() && DelayedDrag->IsDragging() ? KeysByCurve[KeysByCurveIdx].Pivot : GetPivot(Curve, KeyPositions);

			FCurveEditorScreenSpace CurveSpace = View->GetCurveSpace(CurveID);
			const FVector2D ViewSpacePivot = FVector2D(CurveSpace.SecondsToScreen(Pivot.X), CurveSpace.ValueToScreen(Pivot.Y));

			const FLinearColor PivotColor = Curve->GetColor();

			const FVector2D PivotSize = FVector2D(CurveEditorMultiScaleTool::PivotRadius * 2.f, CurveEditorMultiScaleTool::PivotRadius * 2.f);
			const FVector2D PivotOffset = ViewSpacePivot - (PivotSize * .5f);
			FGeometry PivotGeometry = AllottedGeometry.MakeChild(PivotSize, FSlateLayoutTransform(PivotOffset));

			// Draw pivot icon lines
			const double TangentMul = 2.f;
			const double Thickness = 1.5f;
			const FVector2D PivotIconOffset = PivotSize * .5f;
			FSlateDrawElement::MakeSpline(OutDrawElements, PaintOnLayerId, PivotGeometry.ToPaintGeometry(), FVector2D(-CurveEditorMultiScaleTool::PivotRadius, 0.f) + PivotIconOffset, FVector2D(CurveEditorMultiScaleTool::PivotRadius, 0.f) * TangentMul,
				FVector2D(0.f, -CurveEditorMultiScaleTool::PivotRadius) + PivotIconOffset, FVector2D(0.f, -CurveEditorMultiScaleTool::PivotRadius) * TangentMul, Thickness, ESlateDrawEffect::None, PivotColor);
			FSlateDrawElement::MakeSpline(OutDrawElements, PaintOnLayerId, PivotGeometry.ToPaintGeometry(), FVector2D(CurveEditorMultiScaleTool::PivotRadius, 0.f) + PivotIconOffset, FVector2D(-CurveEditorMultiScaleTool::PivotRadius, 0.f) * TangentMul,
				FVector2D(0.f, CurveEditorMultiScaleTool::PivotRadius) + PivotIconOffset, FVector2D(0.f, CurveEditorMultiScaleTool::PivotRadius) * TangentMul, Thickness, ESlateDrawEffect::None, PivotColor);
			FSlateDrawElement::MakeSpline(OutDrawElements, PaintOnLayerId, PivotGeometry.ToPaintGeometry(), FVector2D(0.f, CurveEditorMultiScaleTool::PivotRadius) + PivotIconOffset, FVector2D(0.f, -CurveEditorMultiScaleTool::PivotRadius) * TangentMul,
				FVector2D(-CurveEditorMultiScaleTool::PivotRadius, 0.f) + PivotIconOffset, FVector2D(-CurveEditorMultiScaleTool::PivotRadius, 0.f) * TangentMul, Thickness, ESlateDrawEffect::None, PivotColor);
			FSlateDrawElement::MakeSpline(OutDrawElements, PaintOnLayerId, PivotGeometry.ToPaintGeometry(), FVector2D(0.f, -CurveEditorMultiScaleTool::PivotRadius) + PivotIconOffset, FVector2D(0.f, CurveEditorMultiScaleTool::PivotRadius) * TangentMul,
				FVector2D(CurveEditorMultiScaleTool::PivotRadius, 0.f) + PivotIconOffset, FVector2D(CurveEditorMultiScaleTool::PivotRadius, 0.f) * TangentMul, Thickness, ESlateDrawEffect::None, PivotColor);
		}

		KeysByCurveIdx++;
	}

}

void FCurveEditorMultiScaleTool::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor.IsValid())
	{
		return;
	}

	UpdateBoundingBox();

	FGeometry WidgetGeo = MultiScaleWidget.MakeGeometry(AllottedGeometry);

	if (!DelayedDrag.IsSet())
	{
		MultiScaleWidget.ParentSpaceDragBegin = MultiScaleWidget.Position + MultiScaleWidget.Size * 0.5;
	}

	// Disable options panel if there are no keys selected
	if (CurveEditor->GetSelection().Count() <= 1 && ToolOptionsOnScope != nullptr)
	{
		ToolOptionsOnScope = nullptr;
		OnOptionsRefreshDelegate.Broadcast();
	}
	else if (CurveEditor->GetSelection().Count() > 1 && ToolOptionsOnScope == nullptr)
	{
		ToolOptionsOnScope = MakeShared<FStructOnScope>(FMultiScaleToolOptions::StaticStruct(), (uint8*)&ToolOptions);
		OnOptionsRefreshDelegate.Broadcast();
	}

	// Keyboard shortcuts to set pivot type
	if (FSlateApplication::Get().GetModifierKeys().IsLeftShiftDown())
	{
		ToolOptions.PivotType = EMultiScalePivotType::FirstKey;
	}
	else if (FSlateApplication::Get().GetModifierKeys().IsRightShiftDown())
	{
		ToolOptions.PivotType = EMultiScalePivotType::LastKey;
	}
}

FReply FCurveEditorMultiScaleTool::OnMouseButtonDown(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor.IsValid())
	{
		return FReply::Unhandled();
	}

	DelayedDrag.Reset();
	if (CurveEditor->GetSelection().Count() > 1 && MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		FGeometry WidgetGeo = MultiScaleWidget.MakeGeometry(MyGeometry);
		EMultiScaleAnchorFlags HitWidgetFlags = MultiScaleWidget.GetAnchorFlagsForMousePosition(WidgetGeo, 0.f, 0.f, MouseEvent.GetScreenSpacePosition());
		if (HitWidgetFlags != EMultiScaleAnchorFlags::None)
		{
			// Start a Delayed Drag
			DelayedDrag = FDelayedDrag(MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()), MouseEvent.GetEffectingButton());

			return FReply::Handled().PreventThrottling();
		}
	}
	return FReply::Unhandled();
}

FReply FCurveEditorMultiScaleTool::OnMouseButtonDoubleClick(TSharedRef<SWidget> OwningWidget, const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	return FReply::Unhandled();
}

FReply FCurveEditorMultiScaleTool::OnMouseButtonUp(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = FReply::Handled();
	if (DelayedDrag.IsSet())
	{
		if (DelayedDrag->IsDragging())
		{
			OnDragEnd();

			// Only return handled if we actually started a drag
			Reply.ReleaseMouseCapture();
		}

		DelayedDrag.Reset();
		return Reply;
	}

	return FReply::Unhandled();
}

FReply FCurveEditorMultiScaleTool::OnMouseMove(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor.IsValid())
	{
		return FReply::Unhandled();
	}

	// Update the Hover State of the widget.
	if (!DelayedDrag.IsSet())
	{
		FGeometry WidgetGeo = MultiScaleWidget.MakeGeometry(MyGeometry);

		EMultiScaleAnchorFlags HitWidgetFlags = MultiScaleWidget.GetAnchorFlagsForMousePosition(WidgetGeo, 0.f, 0.f, MouseEvent.GetScreenSpacePosition());
		MultiScaleWidget.SelectedAnchorFlags = HitWidgetFlags;
	}

	const bool bXSliderHovered = (MultiScaleWidget.SelectedAnchorFlags & EMultiScaleAnchorFlags::XSlider) != EMultiScaleAnchorFlags::None;
	const bool bYSliderHovered = (MultiScaleWidget.SelectedAnchorFlags & EMultiScaleAnchorFlags::YSlider) != EMultiScaleAnchorFlags::None;

	if (DelayedDrag.IsSet())
	{
		FReply Reply = FReply::Handled();

		const FVector2D LocalMousePosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
		if (DelayedDrag->IsDragging())
		{
			OnDrag(MouseEvent, LocalMousePosition);
		}
		else if (DelayedDrag->AttemptDragStart(MouseEvent))
		{
			OnDragStart();
			MultiScaleWidget.ParentSpaceDragBegin.X = bXSliderHovered ? LocalMousePosition.X : MultiScaleWidget.ParentSpaceDragBegin.X;
			MultiScaleWidget.ParentSpaceDragBegin.Y = bYSliderHovered ? LocalMousePosition.Y : MultiScaleWidget.ParentSpaceDragBegin.Y;
			// Steal the capture, as we're now the authoritative widget in charge of a mouse-drag operation
			Reply.CaptureMouse(OwningWidget);
		}

		return Reply;
	}

	return FReply::Unhandled();
}

void FCurveEditorMultiScaleTool::OnFocusLost(const FFocusEvent& InFocusEvent)
{
	// We need to end our drag if we lose Window focus to close the transaction, otherwise alt-tabbing while dragging
	// can cause a transaction to get stuck open.
	StopDragIfPossible();
}

void FCurveEditorMultiScaleTool::BindCommands(TSharedRef<FUICommandList> CommandBindings)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor)
	{
		FIsActionChecked MultiScaleToolIsActive = FIsActionChecked::CreateSP(CurveEditor.ToSharedRef(), &FCurveEditor::IsToolActive, ToolID);
		FExecuteAction ActivateMultiScaleTool = FExecuteAction::CreateSP(CurveEditor.ToSharedRef(), &FCurveEditor::MakeToolActive, ToolID);

		CommandBindings->MapAction(FCurveEditorToolCommands::Get().ActivateMultiScaleTool, ActivateMultiScaleTool, FCanExecuteAction(), MultiScaleToolIsActive);
	}
}

void FCurveEditorMultiScaleTool::OnToolOptionsUpdated(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor.IsValid())
	{
		return;
	}

	OnDragStart();

	TArray<FCurveModelID> CurveKeys;
	CurveEditor->GetSelection().GetAll().GetKeys(CurveKeys);
	if (CurveKeys.Num() < 1)
	{
		return;
	}

	// grab the view for the first curve, if there are multiple curves it is assumed they all have the same curve space
	const SCurveEditorView* View = CurveEditor->FindFirstInteractiveView(CurveKeys[0]);
	if (!View)
	{
		return;
	}

	FCurveEditorScreenSpace CurveSpace = View->GetCurveSpace(CurveKeys[0]);

	const FVector2D ScaleAmount = FVector2D(ToolOptions.XScale - 1.f, ToolOptions.YScale - 1.f);
	
	ScaleUnique(ScaleAmount, true, true);

	ToolOptions.XScale = 1.f;
	ToolOptions.YScale = 1.f;
	ActiveTransaction.Reset();
	TangentFlattener.Reset();
}

FText FCurveEditorMultiScaleTool::GetLabel() const
{
	return FCurveEditorToolCommands::Get().ActivateMultiScaleTool->GetLabel();
}

FText FCurveEditorMultiScaleTool::GetDescription() const
{
	return FCurveEditorToolCommands::Get().ActivateMultiScaleTool->GetDescription();
}

FSlateIcon FCurveEditorMultiScaleTool::GetIcon() const
{
	return FCurveEditorToolCommands::Get().ActivateMultiScaleTool->GetIcon();
}

void FCurveEditorMultiScaleTool::OnDragStart()
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor.IsValid())
	{
		return;
	}

	using namespace UE::CurveEditor;
	ActiveTransaction = MakeUnique<FScopedCurveChange>(
		FCurvesSnapshotBuilder(
			CurveEditor, ECurveChangeFlags::KeyAttributes | ECurveChangeFlags::MoveKeysAndRemoveStackedKeys
			).TrackSelectedCurves(),
		LOCTEXT("CurveEditorMultiScaleToolTransaction", "Multi Scale Key(s)")
		);
	TangentFlattener.Emplace(*CurveEditor);

	// We need to cache our key data because all of our calculations have to be relative to the starting data and not the current per-frame data.
	KeysByCurve.Reset();
	bool bMinMaxIsSet = false; // Will be better cached than TOptional and we only need to check this flag here.

	for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : CurveEditor->GetSelection().GetAll())
	{
		FCurveModelID CurveID = Pair.Key;
		FCurveModel*  Curve = CurveEditor->FindCurve(CurveID);

		if (ensureAlways(Curve))
		{
			TArrayView<const FKeyHandle> Handles = Pair.Value.AsArray();

			FKeyData& KeyData = KeysByCurve.Emplace_GetRef(CurveID);
			KeyData.Handles = TArray<FKeyHandle>(Handles.GetData(), Handles.Num());

			KeyData.StartKeyPositions.SetNumZeroed(KeyData.Handles.Num());
			Curve->GetKeyPositions(KeyData.Handles, KeyData.StartKeyPositions);
			if (KeyData.StartKeyPositions.Num() == 0)
			{
				continue;
			}
			for (const FKeyPosition& KeyPosition : KeyData.StartKeyPositions)
			{
				if (bMinMaxIsSet)
				{
					if (KeyPosition.InputValue < InputMinMax.GetLowerBoundValue())
					{
						InputMinMax.SetLowerBoundValue(KeyPosition.InputValue);
					}
					else if (KeyPosition.InputValue > InputMinMax.GetUpperBoundValue())
					{
						InputMinMax.SetUpperBoundValue(KeyPosition.InputValue);
					}
				}
				else
				{
					bMinMaxIsSet = true;
					InputMinMax = TRange<double>(KeyPosition.InputValue);
				}
			}

			KeyData.Pivot = GetPivot(Curve, KeyData.StartKeyPositions);
		}
	}

	MultiScaleWidget.StartSize = MultiScaleWidget.Size;
	MultiScaleWidget.StartPosition = MultiScaleWidget.Position;
	DragDelta = FVector2D::ZeroVector;
}
	
void FCurveEditorMultiScaleTool::OnDrag(const FPointerEvent& InMouseEvent, const FVector2D& InMousePosition)
{
	check(DelayedDrag)
 	DragDelta = InMousePosition - DelayedDrag.GetValue().GetInitialPosition();
	const FVector2D Scale = (DragDelta.GetValue() / (MultiScaleWidget.StartSize * .5f)) * FVector2D(1.f, -1.f);

	const bool bXSliderHovered = (MultiScaleWidget.SelectedAnchorFlags & EMultiScaleAnchorFlags::XSlider) != EMultiScaleAnchorFlags::None;
	const bool bYSliderHovered = (MultiScaleWidget.SelectedAnchorFlags & EMultiScaleAnchorFlags::YSlider) != EMultiScaleAnchorFlags::None;

	if (bXSliderHovered)
	{
		ToolOptions.XScale = 1.f + Scale.X;
		DragDelta->Y = 0.f;
		ScaleUnique(FVector2D(Scale.X, 0.f), true, false);
	}
	else if (bYSliderHovered)
	{
		ToolOptions.YScale = 1.f + Scale.Y;
		DragDelta->X = 0.f;
		ScaleUnique(FVector2D(0.f, Scale.Y), false, true);
	}
}

void FCurveEditorMultiScaleTool::OnDragEnd()
{
	// Call SetKeyPositions again with EPropertyChangeType::ValueSet to trigger any follow up operations, like getting rid of stacked keys...
	ApplyLastKeyPositions();
	// ... then finalize the transaction
	ActiveTransaction.Reset();
	
	KeysByCurve.Reset(); // User might not use our tool anymore - clean-up any cached memory.
	DragDelta = TOptional<FVector2D>();
	ToolOptions.XScale = 1.0f;
	ToolOptions.YScale = 1.0f;
}

void FCurveEditorMultiScaleTool::StopDragIfPossible()
{
	if (DelayedDrag.IsSet())
	{
		if (DelayedDrag->IsDragging())
		{
			OnDragEnd();
		}

		DelayedDrag.Reset();
	}
}

void FCurveEditorMultiScaleTool::UpdateBoundingBox()
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor.IsValid())
	{
		return;
	}
	// We need to look at all selected keys and get their positions relative to the widget view. This lets us put the bounding box around the
	// current selection, even if it goes off-screen (which it may).

	TOptional<FVector2D> MinValue;
	TOptional<FVector2D> MaxValue;

	FSlateLayoutTransform AbsoluteToContainer = CurveEditor->GetPanel()->GetViewContainerGeometry().GetAccumulatedLayoutTransform();

	const TMap<FCurveModelID, FKeyHandleSet>& SelectedKeySet = CurveEditor->GetSelection().GetAll();
	for (const TPair<FCurveModelID, FKeyHandleSet>& Pair : SelectedKeySet)
	{
		const SCurveEditorView* View = CurveEditor->FindFirstInteractiveView(Pair.Key);
		if (!View)
		{
			continue;
		}

		// A newly created view may have a zero-size until the next tick which is a problem if
		// we ask the View for it's curve space, so we skip over it until it has a size.
		if (View->GetCachedGeometry().GetLocalSize() == FVector2D::ZeroVector)
		{
			continue;
		}

		FCurveModel* CurveModel = CurveEditor->FindCurve(Pair.Key);
		check(CurveModel);

		TArrayView<const FKeyHandle> KeyHandles = Pair.Value.AsArray();

		TArray<FKeyPosition> KeyPositions;
		KeyPositions.SetNumUninitialized(KeyHandles.Num());
		CurveModel->GetKeyPositions(KeyHandles, KeyPositions);

		FCurveEditorScreenSpace CurveSpace = View->GetCurveSpace(Pair.Key);
		FSlateLayoutTransform InnerToOuterTransform = Concatenate(View->GetCachedGeometry().GetAccumulatedLayoutTransform(), AbsoluteToContainer.Inverse());

		for (int32 i = 0; i < KeyPositions.Num(); i++)
		{
			const FVector2D ViewSpaceLocation = FVector2D(CurveSpace.SecondsToScreen(KeyPositions[i].InputValue), CurveSpace.ValueToScreen(KeyPositions[i].OutputValue));
			const FVector2D PanelSpaceLocation = InnerToOuterTransform.TransformPoint(ViewSpaceLocation);

			if (!MinValue.IsSet())
			{
				MinValue = PanelSpaceLocation;
			}

			if (!MaxValue.IsSet())
			{
				MaxValue = PanelSpaceLocation;
			}

			MinValue = FVector2D::Min(MinValue.GetValue(), PanelSpaceLocation);
			MaxValue = FVector2D::Max(MaxValue.GetValue(), PanelSpaceLocation);
		}
	}

	if (MinValue.IsSet() && MaxValue.IsSet())
	{
		FVector2D MarqueeSize = MaxValue.GetValue() - MinValue.GetValue();
		FVector2D Offset = FVector2D::ZeroVector;

		MultiScaleWidget.BoundsSize = MarqueeSize;
		MultiScaleWidget.BoundsPosition = MinValue.GetValue();

		// Enforce a minimum size for single time/value selections.
		if (MarqueeSize.X < 8.f)
		{
			MarqueeSize.X = 30;
			Offset.X = MarqueeSize.X / 2.f;
		}
		if (MarqueeSize.Y < 8.f)
		{
			MarqueeSize.Y = 30;
			Offset.Y = MarqueeSize.Y / 2.f;
		}

		MultiScaleWidget.Visible = true;
		MultiScaleWidget.Size = MarqueeSize;
		MultiScaleWidget.Position = MinValue.GetValue() - Offset;
	}
	else
	{

		// No selection, no bounding box.
		MultiScaleWidget.Visible = false;
		MultiScaleWidget.Size = FVector2D::ZeroVector;
		MultiScaleWidget.Position = FVector2D::ZeroVector;
	}
}


void FCurveEditorMultiScaleTool::ScaleUnique(const FVector2D& InChangeAmount, const bool bInAffectsX, const bool bInAffectsY)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor.IsValid())
	{
		return;
	}
	
	// Rescaling is where things get tricky. If a user is selecting an edge we scale on one axis at once, and if they select a corner we scale on two axis at once.
	// If they press alt, we scale relative to the center (instead of relative to the opposite corner). Because these are all bit-flag'd together we can implement
	// this generically by scaling coordinates relative to an arbitrary center
	const FVector2D PercentChanged = FVector2D(1.0f, 1.0f) + InChangeAmount; // ie: 5 pixel change on a 100 wide gives you 1.05
	
	// We now know if we need to affect both X and Y, and we know where we're scaling from. Now we can loop through the keys and actually modify their positions.
	// We perform the scale on both axis (for simplicity) and then read which axis it should effect before assigning it back to the key position.
	for (FKeyData& KeyData : KeysByCurve)
	{
		const SCurveEditorView* View = CurveEditor->FindFirstInteractiveView(KeyData.CurveID);
		if (!View)
		{
			continue;
		}

		FCurveModel* CurveModel = CurveEditor->FindCurve(KeyData.CurveID);
		check(CurveModel);

		const double CurveSpaceCenterInput = KeyData.Pivot.X;
		const double CurveSpaceCenterOutput = KeyData.Pivot.Y;

		TArray<FKeyPosition>& LastDraggedKeyPositions = KeyData.LastDraggedKeyPositions;
		LastDraggedKeyPositions.Empty(KeyData.StartKeyPositions.Num());

		FCurveSnapMetrics SnapMetrics = CurveEditor->GetCurveSnapMetrics(KeyData.CurveID);

		for (FKeyPosition StartPosition : KeyData.StartKeyPositions)
		{
			// Step 1 is to rescale the position of the key by the percentage change on each axis.
			const double ScaledInput = (StartPosition.InputValue - CurveSpaceCenterInput) * PercentChanged.X; // *InputMulSign;
			const double ScaledOutput = (StartPosition.OutputValue - CurveSpaceCenterOutput) * PercentChanged.Y; // *OutputMulSign;

			// Step 2 is to subtract it from the center position so we support scaling from places other than zero
			const double NewInput = CurveSpaceCenterInput + ScaledInput;
			const double NewOutput = CurveSpaceCenterOutput + ScaledOutput;

			// Snap the new values to the grid. We calculate both X and Y changes for ease of programming above and just limit which one it applies to.
			// This includes snapping, otherwise dragging on an edge can cause it to snap on the opposite axis.
			if (bInAffectsX)
			{
				StartPosition.InputValue = View->IsTimeSnapEnabled() ? SnapMetrics.SnapInputSeconds(NewInput) : NewInput;
			}
			if (bInAffectsY)
			{
				StartPosition.OutputValue = View->IsValueSnapEnabled() ? SnapMetrics.SnapOutput(NewOutput) : NewOutput;
			}

			LastDraggedKeyPositions.Add(StartPosition);
		}

		CurveModel->SetKeyPositions(KeyData.Handles, LastDraggedKeyPositions, EPropertyChangeType::Interactive);
	}
	
	// This recalculates the user specified tangents based on the distance moved. We'll flatten the curves depending on the resulting selection's height.
	if (bInAffectsY)
	{
		const bool bIsMirroring = PercentChanged.Y < 0.0;
		TangentFlattener->ComputeMirroringParallel(*CurveEditor, bIsMirroring);
	}
}

void FCurveEditorMultiScaleTool::ApplyLastKeyPositions()
{
	const TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor.IsValid())
	{
		return;
	}
	
	for (const FKeyData& KeyData : KeysByCurve)
	{
		if (FCurveModel* Curve = CurveEditor->FindCurve(KeyData.CurveID))
		{
			Curve->SetKeyPositions(KeyData.Handles, KeyData.LastDraggedKeyPositions, EPropertyChangeType::ValueSet);
		}
	}
}

FVector2D FCurveEditorMultiScaleTool::GetPivot(FCurveModel* InCurve, const TArray<FKeyPosition>& InKeyPositions) const
{
	FVector2D Pivot = FVector2D::ZeroVector;
	if (InKeyPositions.Num() == 0)
	{
		return Pivot;
	}

	switch (ToolOptions.PivotType)
	{
	case EMultiScalePivotType::Average:
		check(InKeyPositions.Num() != 0);
		Pivot = FVector2D::ZeroVector;
		for (const FKeyPosition& KeyPosition : InKeyPositions)
		{
			Pivot += FVector2D(KeyPosition.InputValue, KeyPosition.OutputValue);
		}
		Pivot /= InKeyPositions.Num();
		return Pivot;
	case EMultiScalePivotType::BoundCenter:
		double MinTime, MaxTime, MinValue, MaxValue;
		MinTime = Algo::MinElementBy(InKeyPositions, [](const FKeyPosition& Pos) { return Pos.InputValue; })->InputValue;
		MaxTime = Algo::MaxElementBy(InKeyPositions, [](const FKeyPosition& Pos) { return Pos.InputValue; })->InputValue;
		MinValue = Algo::MinElementBy(InKeyPositions, [](const FKeyPosition& Pos) { return Pos.OutputValue; })->OutputValue;
		MaxValue = Algo::MaxElementBy(InKeyPositions, [](const FKeyPosition& Pos) { return Pos.OutputValue; })->OutputValue;
		Pivot.X = (MinTime + MaxTime) * .5f;
		Pivot.Y = (MinValue + MaxValue) * .5f;
		return Pivot;
	case EMultiScalePivotType::FirstKey:
		const FKeyPosition* FirstKey;
		FirstKey = Algo::MinElementBy(InKeyPositions, [](const FKeyPosition& Pos) { return Pos.InputValue; });
		Pivot.X = FirstKey->InputValue;
		Pivot.Y = FirstKey->OutputValue;
		return Pivot;
	case EMultiScalePivotType::LastKey:
		const FKeyPosition* LastKey;
		LastKey = Algo::MaxElementBy(InKeyPositions, [](const FKeyPosition& Pos) { return Pos.InputValue; });
		Pivot.X = LastKey->InputValue;
		Pivot.Y = LastKey->OutputValue;
		return Pivot;
	}
	return Pivot;
}

#undef LOCTEXT_NAMESPACE

