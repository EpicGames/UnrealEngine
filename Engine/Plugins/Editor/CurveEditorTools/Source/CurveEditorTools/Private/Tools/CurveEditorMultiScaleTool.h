// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveDataAbstraction.h"
#include "CurveEditorTypes.h"
#include "Framework/DelayedDrag.h"
#include "ICurveEditorToolExtension.h"
#include "Misc/Mirror/TangentSelectionFlattener.h"
#include "Modification/Utils/ScopedCurveChange.h"
#include "UObject/StructOnScope.h"
#include "CurveEditorMultiScaleTool.generated.h"

class FCurveEditor;
class FCurveModel;
struct FCurveModelID;
struct FKeyHandle;

enum class EMultiScaleAnchorFlags : uint8
{
	None = 0x00,
	XSlider = 0x01,
	YSlider = 0x02,
	XSidebar = 0x04,
	YSidebar = 0x08
};

ENUM_CLASS_FLAGS(EMultiScaleAnchorFlags);

UENUM()
enum class EMultiScalePivotType : uint8
{
	Average,
	BoundCenter,
	FirstKey,
	LastKey
};

struct FCurveEditorMultiScaleWidget
{
	FCurveEditorMultiScaleWidget()
	{
		SelectedAnchorFlags = EMultiScaleAnchorFlags::None;
		Visible = false;
	}

public:
	EMultiScaleAnchorFlags SelectedAnchorFlags;

	FGeometry MakeGeometry(const FGeometry& InWidgetGeometry) const
	{
		return InWidgetGeometry.MakeChild(Size, FSlateLayoutTransform(Position));
	}

	EMultiScaleAnchorFlags GetAnchorFlagsForMousePosition(const FGeometry& InWidgetGeometry, float InUIXScaleFactor, float InUIYScaleFactor, const FVector2D& InMouseScreenPosition) const;

	void GetXSidebarGeometry(const FGeometry& InWidgetGeometry, const FGeometry& InViewGeometry, bool bInSelected, FGeometry& OutSidebar) const;
	void GetYSidebarGeometry(const FGeometry& InWidgetGeometry, const FGeometry& InViewGeometry, bool bInSelected, FGeometry& OutSidebar) const;
	
	void GetXSliderGeometry(const FGeometry& InWidgetGeometry, float InUIScaleFactor, FGeometry& OutSlider) const;
	void GetYSliderGeometry(const FGeometry& InWidgetGeometry, float InUIScaleFactor, FGeometry& OutSlider) const;

	FVector2D Size;
	FVector2D Position;
	bool Visible;

	FVector2D ParentSpaceDragBegin;

	// stores the physical bounds instead of the marquee bounds, for scale calculations
	FVector2D BoundsSize;
	FVector2D BoundsPosition;

	FVector2D StartSize;
	FVector2D StartPosition;
};

USTRUCT()
struct FMultiScaleToolOptions
{
	GENERATED_BODY()

	UPROPERTY(Transient, EditAnywhere, Category = ToolOptions)
	float XScale = 1.f;

	UPROPERTY(Transient, EditAnywhere, Category = ToolOptions)
	float YScale = 1.f;
	
	UPROPERTY(Transient, EditAnywhere, Category = ToolOptions)
	EMultiScalePivotType PivotType = EMultiScalePivotType::Average;
};

class FCurveEditorMultiScaleTool : public ICurveEditorToolExtension
{
public:
	FCurveEditorMultiScaleTool(TWeakPtr<FCurveEditor> InCurveEditor)
		: WeakCurveEditor(InCurveEditor)
	{
		ToolOptions.XScale = ToolOptions.YScale = 1.f;
		ToolOptions.PivotType = EMultiScalePivotType::BoundCenter;
		ToolOptionsOnScope = MakeShared<FStructOnScope>(FMultiScaleToolOptions::StaticStruct(), (uint8*)&ToolOptions);
	}

	// ICurveEditorToolExtension
	virtual void OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 PaintOnLayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnMouseButtonDown(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonDoubleClick(TSharedRef<SWidget> OwningWidget, const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnMouseButtonUp(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnFocusLost(const FFocusEvent& InFocusEvent) override;
	virtual void OnToolActivated() override {}
	virtual void OnToolDeactivated() override {}
	virtual void BindCommands(TSharedRef<FUICommandList> CommandBindings) override;

	TSharedPtr<FStructOnScope> GetToolOptions() const override { return ToolOptionsOnScope; }
	virtual void OnToolOptionsUpdated(const FPropertyChangedEvent& InPropertyChangedEvent) override;
	virtual FText GetLabel() const override;
	virtual FText GetDescription() const override;
	virtual FSlateIcon GetIcon() const override;
	// ~ICurveEditorToolExtension

private:
	void OnDragStart();
	void OnDrag(const FPointerEvent& InMouseEvent, const FVector2D& InMousePosition);
	void OnDragEnd();
	void StopDragIfPossible();
	void UpdateBoundingBox();

	void ScaleUnique(const FVector2D& InChangeAmount, const bool bInAffectsX, const bool bInAffectsY);
	
	/**
	 * Calls SetKeyPositions with FKeyData::LastDraggedKeyPositions and EPropertyChangeType::ValueSet tro trigger any final operations.
	 * 
	 * Some FCurveModels, such as Sequencer's FCurveChannelModel, perform special operations with EPropertyChangeType::ValueSet, e.g. cleansing of
	 * stacked values.
	 * 
	 * Called at end of drag.
	 */
	void ApplyLastKeyPositions();

private:
	/** Weak pointer back to the owning curve editor. */
	TWeakPtr<FCurveEditor> WeakCurveEditor;

	/** The currently open transaction (if any) */
	TUniquePtr<UE::CurveEditor::FScopedCurveChange> ActiveTransaction;
	
	/** Set when attempting to move a drag handle. */
	TOptional<FDelayedDrag> DelayedDrag;
	TOptional<FVector2D> DragDelta;
	/** Squishes the tangents vertically depending on how much we move the keys vertically. */
	TOptional<UE::CurveEditor::FTangentSelectionFlattener> TangentFlattener;

	/** Used to cache selected key data when doing transform operations */
	struct FKeyData
	{
		FKeyData(FCurveModelID InCurveID)
			: CurveID(InCurveID)
		{}

		/** The curve that contains the keys we're dragging */
		FCurveModelID CurveID;
		/** All the handles within a given curve that we are dragging */
		TArray<FKeyHandle> Handles;
		/** The extended key info for each of the above handles */
		TArray<FKeyPosition> StartKeyPositions;
		/** Used in OnEndDrag to send final key updates */
		TArray<FKeyPosition> LastDraggedKeyPositions;
		/** Pivot to scale from */
		FVector2D Pivot;
	};

	/** Key dragging data stored per-curve */
	TArray<FKeyData> KeysByCurve;
	TRange<double> InputMinMax;

	/** UStruct that displays properties to be modified on screen */
	FMultiScaleToolOptions ToolOptions;
	TSharedPtr<FStructOnScope> ToolOptionsOnScope;

	/** Multi select widget to store selection's current state */
	FCurveEditorMultiScaleWidget MultiScaleWidget;

private:
	FVector2D GetPivot(FCurveModel* InCurve, const TArray<FKeyPosition>& InKeyPositions) const;
};
