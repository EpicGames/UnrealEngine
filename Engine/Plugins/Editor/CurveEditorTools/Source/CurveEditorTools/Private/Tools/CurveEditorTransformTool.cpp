// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorTransformTool.h"
#include "CurveEditorToolCommands.h"
#include "CurveEditor.h"
#include "Framework/Commands/UICommandList.h"
#include "Misc/FrameRate.h"
#include "Rendering/DrawElementPayloads.h"
#include "SCurveEditorView.h"
#include "SCurveEditorPanel.h"
#include "CurveModel.h"
#include "UObject/StructOnScope.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveEditorTransformTool)

#define LOCTEXT_NAMESPACE "CurveEditorToolCommands"

namespace CurveEditorTransformTool
{
	constexpr float ScaleCenterRadius = 16.0f;
	constexpr float EdgeAnchorWidth = 13.f;
	constexpr float SoftSelectAnchorWidth = 20.f;
	constexpr float EdgeHighlightAlpha = 0.15f;
	constexpr float MaxFalloffOpacity = 0.6f;
	constexpr int32 FalloffGradientSampleSize = 10;

	constexpr float ArrowMargin = .3f;
	constexpr float ArrowDiagOffset = 5.f;
	constexpr float ArrowOffset = 10.f;
	constexpr float ArrowThreshold = 50.f;
	constexpr float ArrowExtraRightOffset = 12.f;
	constexpr float ArrowExtraTopOffset = 12.f;
}

void FCurveEditorTransformWidget::GetSidebarGeometry(const FGeometry& InWidgetGeometry, FGeometry& OutLeft, FGeometry& OutRight, FGeometry& OutTop, FGeometry& OutBottom) const
{
	const FVector2D SidebarSize = FVector2D(CurveEditorTransformTool::EdgeAnchorWidth, InWidgetGeometry.GetLocalSize().Y - CurveEditorTransformTool::EdgeAnchorWidth);
	const FVector2D SidebarSizeOffset = FVector2D(CurveEditorTransformTool::EdgeAnchorWidth / 2.f, 0.f);
	const FVector2D TopbarSize = FVector2D(InWidgetGeometry.GetLocalSize().X - CurveEditorTransformTool::EdgeAnchorWidth, CurveEditorTransformTool::EdgeAnchorWidth);
	const FVector2D TopbarSizeOffset = FVector2D(0.f, CurveEditorTransformTool::EdgeAnchorWidth / 2.f);

	OutLeft = InWidgetGeometry.MakeChild(SidebarSize, FSlateLayoutTransform(FVector2D(0, CurveEditorTransformTool::EdgeAnchorWidth / 2.f) - SidebarSizeOffset));
	OutRight = InWidgetGeometry.MakeChild(SidebarSize, FSlateLayoutTransform(FVector2D(InWidgetGeometry.GetLocalSize().X, CurveEditorTransformTool::EdgeAnchorWidth / 2.f) - SidebarSizeOffset));
	OutTop = InWidgetGeometry.MakeChild(TopbarSize, FSlateLayoutTransform(FVector2D(CurveEditorTransformTool::EdgeAnchorWidth / 2.f, 0) - TopbarSizeOffset));
	OutBottom = InWidgetGeometry.MakeChild(TopbarSize, FSlateLayoutTransform(FVector2D(CurveEditorTransformTool::EdgeAnchorWidth / 2.f, InWidgetGeometry.GetLocalSize().Y) - TopbarSizeOffset));
}

void FCurveEditorTransformWidget::GetCornerGeometry(const FGeometry& InWidgetGeometry, FGeometry& OutTopLeft, FGeometry& OutTopRight, FGeometry& OutBottomLeft, FGeometry& OutBottomRight) const
{
	const FVector2D CornerSize = FVector2D(CurveEditorTransformTool::EdgeAnchorWidth, CurveEditorTransformTool::EdgeAnchorWidth);
	const FVector2D HalfSizeOffset = FVector2D(CornerSize / 2.f);

	FSlateLayoutTransform TopLeftPosition = FSlateLayoutTransform(FVector2D(0, 0) - HalfSizeOffset);
	FSlateLayoutTransform TopRightPosition = FSlateLayoutTransform(FVector2D(InWidgetGeometry.GetLocalSize().X, 0) - HalfSizeOffset);
	FSlateLayoutTransform BottomLeftPosition = FSlateLayoutTransform(FVector2D(0, InWidgetGeometry.GetLocalSize().Y) - HalfSizeOffset);
	FSlateLayoutTransform BottomRightPosition = FSlateLayoutTransform(FVector2D(InWidgetGeometry.GetLocalSize()) - HalfSizeOffset);

	OutTopLeft = InWidgetGeometry.MakeChild(CornerSize, TopLeftPosition);
	OutTopRight = InWidgetGeometry.MakeChild(CornerSize, TopRightPosition);
	OutBottomLeft = InWidgetGeometry.MakeChild(CornerSize, BottomLeftPosition);
	OutBottomRight = InWidgetGeometry.MakeChild(CornerSize, BottomRightPosition);
}

void FCurveEditorTransformWidget::GetCenterGeometry(const FGeometry& InWidgetGeometry, FGeometry& OutCenter) const
{
	const FVector2D CenterSize = InWidgetGeometry.GetLocalSize() - FVector2D(CurveEditorTransformTool::EdgeAnchorWidth, CurveEditorTransformTool::EdgeAnchorWidth);
	const FVector2D CenterOffset = FVector2D(CurveEditorTransformTool::EdgeAnchorWidth / 2.f, CurveEditorTransformTool::EdgeAnchorWidth / 2.f);

	OutCenter = InWidgetGeometry.MakeChild(CenterSize, FSlateLayoutTransform(CenterOffset));
}

void FCurveEditorTransformWidget::GetFalloffGeometry(const FGeometry& InWidgetGeometry, float FalloffHeight, float FalloffWidth, FGeometry& OutTopLeft, FGeometry& OutTopRight, FGeometry& OutLeft, FGeometry& OutRight) const
{
	const FVector2D CornerSize = FVector2D(CurveEditorTransformTool::SoftSelectAnchorWidth, CurveEditorTransformTool::SoftSelectAnchorWidth);
	const FVector2D HalfSizeOffset = FVector2D(CornerSize / 2.f);
	const float HeightDiff = InWidgetGeometry.GetLocalSize().Y * FalloffHeight;
	const float WidthDiff = InWidgetGeometry.GetLocalSize().X * 0.5f * FalloffWidth;
	FSlateLayoutTransform TopLeftPosition = FSlateLayoutTransform(FVector2D(InWidgetGeometry.GetLocalSize().X * 0.5f - WidthDiff, 0) - HalfSizeOffset);
	FSlateLayoutTransform TopRightPosition = FSlateLayoutTransform(FVector2D(InWidgetGeometry.GetLocalSize().X * 0.5f + WidthDiff, 0) - HalfSizeOffset);
	FSlateLayoutTransform BottomLeftPosition = FSlateLayoutTransform(FVector2D(0, InWidgetGeometry.GetLocalSize().Y - HeightDiff) - HalfSizeOffset);
	FSlateLayoutTransform BottomRightPosition = FSlateLayoutTransform(FVector2D(InWidgetGeometry.GetLocalSize().X, InWidgetGeometry.GetLocalSize().Y - HeightDiff) - HalfSizeOffset);

	OutTopLeft = InWidgetGeometry.MakeChild(CornerSize, TopLeftPosition);
	OutTopRight = InWidgetGeometry.MakeChild(CornerSize, TopRightPosition);
	OutLeft = InWidgetGeometry.MakeChild(CornerSize, BottomLeftPosition);
	OutRight = InWidgetGeometry.MakeChild(CornerSize, BottomRightPosition);
}

void FCurveEditorTransformWidget::GetScaleCenterGeometry(const FGeometry& InWidgetGeometry, FVector2D ScaleCenter, FGeometry& OutScaleCenterGeometry) const
{
	const FVector2D CenterSize = FVector2D(CurveEditorTransformTool::ScaleCenterRadius * 2.f, CurveEditorTransformTool::ScaleCenterRadius * 2.f);
	const FVector2D CenterOffset = (InWidgetGeometry.GetLocalSize() * .5f) + (BoundsSize * (ScaleCenter - 0.5f)) - (CenterSize * .5f);
	OutScaleCenterGeometry = InWidgetGeometry.MakeChild(CenterSize, FSlateLayoutTransform(CenterOffset));
}

void FCurveEditorTransformWidget::GetCenterIndicatorGeometry(const FGeometry& InWidgetGeometry, FGeometry& OutCenterIndicatorGeometry) const
{
	const FVector2D IndicatorSize = FVector2D(CurveEditorTransformTool::ScaleCenterRadius * 2.f, CurveEditorTransformTool::ScaleCenterRadius * 2.f);
	const FVector2D IndicatorOffset = (InWidgetGeometry.GetLocalSize() * .5f) - (IndicatorSize * 0.5f);
	OutCenterIndicatorGeometry = InWidgetGeometry.MakeChild(IndicatorSize, FSlateLayoutTransform(IndicatorOffset));
}

ECurveEditorAnchorFlags FCurveEditorTransformWidget::GetAnchorFlagsForMousePosition(const FGeometry& InWidgetGeometry, float  FalloffHeight, float FalloffWidth, const FVector2D& RelativeScaleCenter, const FVector2D& InMouseScreenPosition) const
{
	// We store a geometry to represent each different region, updated on Tick. We check if the mouse
	// overlaps a region and update the selection anchors depending on which region you're hovering in.
	ECurveEditorAnchorFlags  OutFlags = ECurveEditorAnchorFlags::None;

	// If holding ctrl we test for falloff first and then bail if getting hit 
	if (FSlateApplication::Get().GetModifierKeys().IsControlDown())
	{
		FGeometry TopLeftFalloffGeometry, TopRightFalloffGeometry, LeftFalloffGeometry, RightFalloffGeometry;
		GetFalloffGeometry(InWidgetGeometry, FalloffHeight, FalloffWidth, TopLeftFalloffGeometry, TopRightFalloffGeometry, LeftFalloffGeometry, RightFalloffGeometry);

		if (TopLeftFalloffGeometry.IsUnderLocation(InMouseScreenPosition))
		{
			OutFlags |= ECurveEditorAnchorFlags::FalloffTopLeft;
		}
		if (TopRightFalloffGeometry.IsUnderLocation(InMouseScreenPosition))
		{
			OutFlags |= ECurveEditorAnchorFlags::FalloffTopRight;
		}
		if (LeftFalloffGeometry.IsUnderLocation(InMouseScreenPosition))
		{
			OutFlags |= ECurveEditorAnchorFlags::FalloffLeft;
		}
		if (RightFalloffGeometry.IsUnderLocation(InMouseScreenPosition))
		{
			OutFlags |= ECurveEditorAnchorFlags::FalloffRight;
		}
		if (OutFlags != ECurveEditorAnchorFlags::None)
		{
			return OutFlags;
		}
	}

	FGeometry ScaleCenterGeometry;
	GetScaleCenterGeometry(InWidgetGeometry, RelativeScaleCenter, ScaleCenterGeometry);

	if (ScaleCenterGeometry.IsUnderLocation(InMouseScreenPosition))
	{
		OutFlags |= ECurveEditorAnchorFlags::ScaleCenter;
	}

	FGeometry LeftSidebarGeometry, RightSidebarGeometry, TopSidebarGeometry, BottomSidebarGeometry;
	FGeometry TopLeftCornerGeometry, TopRightCornerGeometry, BottomLeftCornerGeometry, BottomRightCornerGeometry;

	GetSidebarGeometry(InWidgetGeometry, LeftSidebarGeometry, RightSidebarGeometry, TopSidebarGeometry, BottomSidebarGeometry);
	GetCornerGeometry(InWidgetGeometry, TopLeftCornerGeometry, TopRightCornerGeometry, BottomLeftCornerGeometry, BottomRightCornerGeometry);

	// Deflate the supplied widget by the size of our sidebars so they don't overlap before doing the center check.
	{
		FGeometry CenterGeometry;
		GetCenterGeometry(InWidgetGeometry, CenterGeometry);

		if (CenterGeometry.IsUnderLocation(InMouseScreenPosition))
		{
			OutFlags |= ECurveEditorAnchorFlags::Center;
		}
	}

	if (LeftSidebarGeometry.IsUnderLocation(InMouseScreenPosition))
	{
		OutFlags |= ECurveEditorAnchorFlags::Left;
	}
	if (RightSidebarGeometry.IsUnderLocation(InMouseScreenPosition))
	{
		OutFlags |= ECurveEditorAnchorFlags::Right;
	}
	if (TopSidebarGeometry.IsUnderLocation(InMouseScreenPosition))
	{
		OutFlags |= ECurveEditorAnchorFlags::Top;
	}
	if (BottomSidebarGeometry.IsUnderLocation(InMouseScreenPosition))
	{
		OutFlags |= ECurveEditorAnchorFlags::Bottom;
	}

	if (TopLeftCornerGeometry.IsUnderLocation(InMouseScreenPosition))
	{
		OutFlags |= ECurveEditorAnchorFlags::Top | ECurveEditorAnchorFlags::Left;
	}
	if (TopRightCornerGeometry.IsUnderLocation(InMouseScreenPosition))
	{
		OutFlags |= ECurveEditorAnchorFlags::Top | ECurveEditorAnchorFlags::Right;
	}
	if (BottomLeftCornerGeometry.IsUnderLocation(InMouseScreenPosition))
	{
		OutFlags |= ECurveEditorAnchorFlags::Bottom | ECurveEditorAnchorFlags::Left;
	}
	if (BottomRightCornerGeometry.IsUnderLocation(InMouseScreenPosition))
	{
		OutFlags |= ECurveEditorAnchorFlags::Bottom | ECurveEditorAnchorFlags::Right;
	}

	return OutFlags;
}

void FCurveEditorTransformTool::OnToolActivated()
{
	// set the new tool option values
	UpdateToolOptions();
}

void FCurveEditorTransformTool::OnToolDeactivated()
{
	
}

void FCurveEditorTransformTool::UpdateMarqueeBoundingBox()
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor.IsValid())
	{
		return;
	}
	// We need to look at all selected keys and get their positions relative to the widget view. This lets us put the bounding box around the
	// current selection, even if it goes off-screen (which it may).

	TOptional<FVector2D> MinValue;
	TOptional<FVector2D> MaxValue;

	FSlateLayoutTransform AbsoluteToContainer = CurveEditor->GetPanel()->GetViewContainerGeometry().GetAccumulatedLayoutTransform();

	const TMap<FCurveModelID, FKeyHandleSet>& SelectedKeySet = CurveEditor->GetSelection().GetAll();
	for (const TPair<FCurveModelID, FKeyHandleSet>& Pair : SelectedKeySet)
	{
		const SCurveEditorView* View = CurveEditor->FindFirstInteractiveView(Pair.Key);
		if (!View)
		{
			continue;
		}

		// A newly created view may have a zero-size until the next tick which is a problem if
		// we ask the View for it's curve space, so we skip over it until it has a size.
		if(View->GetCachedGeometry().GetLocalSize() == FVector2D::ZeroVector)
		{
			continue;
		}

		FCurveModel* CurveModel = CurveEditor->FindCurve(Pair.Key);
		check(CurveModel);

		TArrayView<const FKeyHandle> KeyHandles = Pair.Value.AsArray();

		TArray<FKeyPosition> KeyPositions;
		KeyPositions.SetNumUninitialized(KeyHandles.Num());
		CurveModel->GetKeyPositions(KeyHandles, KeyPositions);

		FCurveEditorScreenSpace CurveSpace = View->GetCurveSpace(Pair.Key);
		FSlateLayoutTransform InnerToOuterTransform = Concatenate(View->GetCachedGeometry().GetAccumulatedLayoutTransform(), AbsoluteToContainer.Inverse());

		for (int32 i = 0; i < KeyPositions.Num(); i++)
		{
			const FVector2D ViewSpaceLocation = FVector2D(CurveSpace.SecondsToScreen(KeyPositions[i].InputValue), CurveSpace.ValueToScreen(KeyPositions[i].OutputValue));
			const FVector2D PanelSpaceLocation = InnerToOuterTransform.TransformPoint(ViewSpaceLocation);

			if (!MinValue.IsSet())
			{
				MinValue = PanelSpaceLocation;
			}

			if (!MaxValue.IsSet())
			{
				MaxValue = PanelSpaceLocation;
			}

			MinValue = FVector2D::Min(MinValue.GetValue(), PanelSpaceLocation);
			MaxValue = FVector2D::Max(MaxValue.GetValue(), PanelSpaceLocation);
		}
	}

	if (MinValue.IsSet() && MaxValue.IsSet())
	{
		FVector2D MarqueeSize = MaxValue.GetValue() - MinValue.GetValue();
		FVector2D Offset = FVector2D::ZeroVector;
		
		TransformWidget.BoundsSize = MarqueeSize;
		TransformWidget.BoundsPosition = MinValue.GetValue();

		// Enforce a minimum size for single time/value selections.
		if (MarqueeSize.X < 8.f)
		{
			MarqueeSize.X = 30;
			Offset.X = MarqueeSize.X / 2.f;
		}
		if (MarqueeSize.Y < 8.f)
		{
			MarqueeSize.Y = 30;
			Offset.Y = MarqueeSize.Y / 2.f;
		}

		TransformWidget.Visible = true;
		TransformWidget.Size = MarqueeSize;
		TransformWidget.Position = MinValue.GetValue() - Offset;
	}
	else
	{

		// No selection, no bounding box.
		TransformWidget.Visible = false;
		TransformWidget.Size = FVector2D::ZeroVector;
		TransformWidget.Position = FVector2D::ZeroVector;
	}
}

void FCurveEditorTransformTool::UpdateToolOptions()
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor.IsValid())
	{
		const FFrameRate DisplayRate = CurveEditor->GetTimeSliderController() ? CurveEditor->GetTimeSliderController()->GetDisplayRate() : FFrameRate();
		const FFrameRate TickResolution = CurveEditor->GetTimeSliderController() ? CurveEditor->GetTimeSliderController()->GetTickResolution() : FFrameRate();

		TArray<FCurveModelID> CurveKeys;
		CurveEditor->GetSelection().GetAll().GetKeys(CurveKeys);
		if (CurveKeys.Num() < 1)
		{
			return;
		}
		const SCurveEditorView* View = CurveEditor->FindFirstInteractiveView(CurveKeys[0]);
		if (!View)
		{
			return;
		}
		FCurveEditorScreenSpace CurveSpace = View->GetCurveSpace(CurveKeys[0]);

		const double LeftBoundSeconds = CurveSpace.ScreenToSeconds(TransformWidget.BoundsPosition.X);
		const double RightBoundSeconds = CurveSpace.ScreenToSeconds(TransformWidget.BoundsPosition.X + TransformWidget.BoundsSize.X);

		ToolOptions.LeftBound = FFrameRate::TransformTime(DisplayRate.AsFrameNumber(LeftBoundSeconds), DisplayRate, TickResolution).CeilToFrame();
		ToolOptions.RightBound = FFrameRate::TransformTime(DisplayRate.AsFrameNumber(RightBoundSeconds), DisplayRate, TickResolution).CeilToFrame();

		ToolOptions.UpperBound = CurveSpace.ScreenToValue(TransformWidget.BoundsPosition.Y);
		ToolOptions.LowerBound = CurveSpace.ScreenToValue(TransformWidget.BoundsPosition.Y + TransformWidget.BoundsSize.Y);

		const double ScaleCenterXSeconds = CurveSpace.ScreenToSeconds(TransformWidget.BoundsPosition.X + TransformWidget.BoundsSize.X * DisplayRelativeScaleCenter.X);

		ToolOptions.ScaleCenterX = FFrameRate::TransformTime(DisplayRate.AsFrameNumber(ScaleCenterXSeconds), DisplayRate, TickResolution).CeilToFrame();
		ToolOptions.ScaleCenterY = CurveSpace.ScreenToValue(TransformWidget.BoundsPosition.Y + TransformWidget.BoundsSize.Y * DisplayRelativeScaleCenter.Y);
	}
}

FReply FCurveEditorTransformTool::OnMouseButtonDown(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	DelayedDrag.Reset();
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		FGeometry WidgetGeo = TransformWidget.MakeGeometry(MyGeometry);
		ECurveEditorAnchorFlags HitWidgetFlags = TransformWidget.GetAnchorFlagsForMousePosition(WidgetGeo, FalloffHeight, FalloffWidth, RelativeScaleCenter, MouseEvent.GetScreenSpacePosition());
		if (HitWidgetFlags != ECurveEditorAnchorFlags::None)
		{
			// Start a Delayed Drag
			DelayedDrag = FDelayedDrag(MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()), MouseEvent.GetEffectingButton());
			
			return FReply::Handled().PreventThrottling();
		}
	}
	return FReply::Unhandled();
}

FReply FCurveEditorTransformTool::OnMouseMove(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return FReply::Unhandled();
	}

	// Update the Hover State of the widget.
	if (!DelayedDrag.IsSet())
	{
		FGeometry WidgetGeo = TransformWidget.MakeGeometry(MyGeometry);
		ECurveEditorAnchorFlags HitWidgetFlags = TransformWidget.GetAnchorFlagsForMousePosition(WidgetGeo, FalloffHeight, FalloffWidth, RelativeScaleCenter, MouseEvent.GetScreenSpacePosition());
		if (CurveEditor->GetSelection().Count() <= 1)
		{
			HitWidgetFlags ^= ECurveEditorAnchorFlags::ScaleCenter;
		}
		TransformWidget.DisplayAnchorFlags = TransformWidget.SelectedAnchorFlags = HitWidgetFlags;
	}

	if (DelayedDrag.IsSet())
	{
		FReply Reply = FReply::Handled();

		if (DelayedDrag->IsDragging())
		{
			const FVector2D LocalMousePosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
			OnDrag(MouseEvent, LocalMousePosition);
		}
		else if (DelayedDrag->AttemptDragStart(MouseEvent))
		{
			LastMousePosition = InitialMousePosition = MouseEvent.GetScreenSpacePosition();
			OnDragStart();

			// Steal the capture, as we're now the authoritative widget in charge of a mouse-drag operation
			Reply.CaptureMouse(OwningWidget);
		}

		return Reply;
	}

	return FReply::Unhandled();
}

FReply FCurveEditorTransformTool::OnMouseButtonUp(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = FReply::Handled();
	if (DelayedDrag.IsSet())
	{
		if (DelayedDrag->IsDragging())
		{
			OnDragEnd();

			// Only return handled if we actually started a drag
			Reply.ReleaseMouseCapture();
		}
		DelayedDrag.Reset();
		return Reply;
	}

	return FReply::Unhandled();
}

void FCurveEditorTransformTool::OnFocusLost(const FFocusEvent& InFocusEvent)
{
	// We need to end our drag if we lose Window focus to close the transaction, otherwise alt-tabbing while dragging
	// can cause a transaction to get stuck open.
	StopDragIfPossible();
}

void FCurveEditorTransformTool::OnToolOptionsUpdated(const FPropertyChangedEvent& PropertyChangedEvent)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return;
	}

	// call OnDragStart() to update the initial key positions in KeysByCurve for ScaleFrom() to scale
	OnDragStart();

	TArray<FCurveModelID> CurveKeys;
	CurveEditor->GetSelection().GetAll().GetKeys(CurveKeys);
	if (CurveKeys.Num() < 1)
	{
		return;
	}

	// grab the view for the first curve, if there are multiple curves it is assumed they all have the same curve space
	const SCurveEditorView* View = CurveEditor->FindFirstInteractiveView(CurveKeys[0]);
	if (!View)
	{
		return;
	}

	const FFrameRate DisplayRate = CurveEditor->GetTimeSliderController() ? CurveEditor->GetTimeSliderController()->GetDisplayRate() : FFrameRate();
	const FFrameRate TickResolution = CurveEditor->GetTimeSliderController() ? CurveEditor->GetTimeSliderController()->GetTickResolution() : FFrameRate();

	FCurveEditorScreenSpace CurveSpace = View->GetCurveSpace(CurveKeys[0]);

	FVector2D ScaleCenter = FVector2D(0.5f, 0.5f);
	FVector2D ScaleDelta = FVector2D(0.0f, 0.0f);
	bool bAffectsX = true, bAffectsY = true;

	if (PropertyChangedEvent.GetMemberPropertyName().IsEqual(GET_MEMBER_NAME_CHECKED(FTransformToolOptions, LeftBound)))
	{
		bAffectsY = false;
		ScaleCenter.X = 1.0f;
		const double LeftBoundSeconds = DisplayRate.AsSeconds(FFrameRate::TransformTime(ToolOptions.LeftBound, TickResolution, DisplayRate));
		ScaleDelta.X = -(CurveSpace.SecondsToScreen(LeftBoundSeconds) - TransformWidget.Position.X);
	}
	else if (PropertyChangedEvent.GetMemberPropertyName().IsEqual(GET_MEMBER_NAME_CHECKED(FTransformToolOptions, UpperBound)))
	{
		bAffectsX = false;
		ScaleCenter.Y = 1.0f;
		ScaleDelta.Y = -(CurveSpace.ValueToScreen(ToolOptions.UpperBound) - TransformWidget.Position.Y);
	}
	else if (PropertyChangedEvent.GetMemberPropertyName().IsEqual(GET_MEMBER_NAME_CHECKED(FTransformToolOptions, RightBound)))
	{
		bAffectsY = false;
		ScaleCenter.X = 0.0f;
		const double RightBoundSeconds = DisplayRate.AsSeconds(FFrameRate::TransformTime(ToolOptions.RightBound, TickResolution, DisplayRate));
		ScaleDelta.X = CurveSpace.SecondsToScreen(RightBoundSeconds) - (TransformWidget.Position.X + TransformWidget.Size.X);
	}
	else if (PropertyChangedEvent.GetMemberPropertyName().IsEqual(GET_MEMBER_NAME_CHECKED(FTransformToolOptions, LowerBound)))
	{
		bAffectsX = false;
		ScaleCenter.Y = 0.0f;
		ScaleDelta.Y = CurveSpace.ValueToScreen(ToolOptions.LowerBound) - (TransformWidget.Position.Y + TransformWidget.Size.Y);
	}
	else if (PropertyChangedEvent.GetMemberPropertyName().IsEqual(GET_MEMBER_NAME_CHECKED(FTransformToolOptions, ScaleCenterX)))
	{
		const double ScaleCenterXSeconds = DisplayRate.AsSeconds(FFrameRate::TransformTime(ToolOptions.ScaleCenterX, TickResolution, DisplayRate));
		const double ViewSpaceX = CurveSpace.SecondsToScreen(ScaleCenterXSeconds);
		const double ViewSpaceXDelta = ViewSpaceX - TransformWidget.BoundsPosition.X;
		RelativeScaleCenter.X = ViewSpaceXDelta / TransformWidget.BoundsSize.X;
	}
	else if (PropertyChangedEvent.GetMemberPropertyName().IsEqual(GET_MEMBER_NAME_CHECKED(FTransformToolOptions, ScaleCenterY)))
	{
		const double ViewSpaceY = CurveSpace.ValueToScreen(ToolOptions.ScaleCenterY);
		const double ViewSpaceYDelta = ViewSpaceY - TransformWidget.BoundsPosition.Y;
		RelativeScaleCenter.Y = ViewSpaceYDelta / TransformWidget.BoundsSize.Y;
	}
	
	DisplayRelativeScaleCenter = RelativeScaleCenter;
	const FVector2D PanelSpaceCenter = TransformWidget.Position + (TransformWidget.Size * ScaleCenter);
	const FVector2D ChangeAmount = (ScaleDelta / TransformWidget.Size);

 	ScaleFrom(PanelSpaceCenter, ChangeAmount, false, bAffectsX, bAffectsY);

	UpdateMarqueeBoundingBox();

	TangentFlattener.Reset();
	ActiveTransaction.Reset();
}

FText FCurveEditorTransformTool::GetLabel() const
{
	return FCurveEditorToolCommands::Get().ActivateTransformTool->GetLabel();
}

FText FCurveEditorTransformTool::GetDescription() const
{
	return FCurveEditorToolCommands::Get().ActivateTransformTool->GetDescription();
}

FSlateIcon FCurveEditorTransformTool::GetIcon() const
{
	return FCurveEditorToolCommands::Get().ActivateTransformTool->GetIcon();
}

void FCurveEditorTransformTool::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 PaintOnLayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	// This Geometry represents the Marquee size, but also the offset into the window
	FGeometry WidgetGeo = TransformWidget.MakeGeometry(AllottedGeometry);
	DrawMarqueeWidget(TransformWidget, Args, WidgetGeo, MyCullingRect, OutDrawElements, PaintOnLayerId, InWidgetStyle, bParentEnabled);
}

void FCurveEditorTransformTool::DrawMarqueeWidget(const FCurveEditorTransformWidget& InTransformWidget, const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InMyCullingRect, FSlateWindowElementList& OutDrawElements, const int32 InPaintOnLayerId, const FWidgetStyle& InWidgetStyle, const bool bInParentEnabled) const
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!InTransformWidget.Visible || !CurveEditor)
	{
		return;
	}

	// Draw the inner marquee dotted rectangle line and the highlight
	{
		const bool bCenterHovered = (InTransformWidget.DisplayAnchorFlags & ECurveEditorAnchorFlags::Center) != ECurveEditorAnchorFlags::None;
		const bool bScaleCenterNotShowing = ((InTransformWidget.DisplayAnchorFlags & ECurveEditorAnchorFlags::ScaleCenter) == ECurveEditorAnchorFlags::None || CurveEditor->GetSelection().Count() == 1);
		const bool bFalloffOn = FSlateApplication::Get().GetModifierKeys().IsControlDown();
		FLinearColor CenterHighlightColor = bCenterHovered && bScaleCenterNotShowing && !bFalloffOn ?
			FLinearColor::White.CopyWithNewOpacity(CurveEditorTransformTool::EdgeHighlightAlpha) : FLinearColor::Transparent;
		FGeometry CenterGeometry;
		InTransformWidget.GetCenterGeometry(InAllottedGeometry, CenterGeometry);

		FSlateDrawElement::MakeBox(OutDrawElements, InPaintOnLayerId, CenterGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("WhiteBrush")), ESlateDrawEffect::None, CenterHighlightColor);
		FSlateDrawElement::MakeBox(OutDrawElements, InPaintOnLayerId, InAllottedGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("MarqueeSelection")));
	}

	// Draw edge highlight regions on mouse hover
	{
		FLinearColor LeftEdgeHighlightColor =	(InTransformWidget.DisplayAnchorFlags == ECurveEditorAnchorFlags::Left)	? FLinearColor::White.CopyWithNewOpacity(CurveEditorTransformTool::EdgeHighlightAlpha) : FLinearColor::Transparent;
		FLinearColor RightEdgeHighlightColor =	(InTransformWidget.DisplayAnchorFlags == ECurveEditorAnchorFlags::Right)	? FLinearColor::White.CopyWithNewOpacity(CurveEditorTransformTool::EdgeHighlightAlpha) : FLinearColor::Transparent;
		FLinearColor TopEdgeHighlightColor =	(InTransformWidget.DisplayAnchorFlags == ECurveEditorAnchorFlags::Top)		? FLinearColor::White.CopyWithNewOpacity(CurveEditorTransformTool::EdgeHighlightAlpha) : FLinearColor::Transparent;
		FLinearColor BottomEdgeHighlightColor =	(InTransformWidget.DisplayAnchorFlags == ECurveEditorAnchorFlags::Bottom)	? FLinearColor::White.CopyWithNewOpacity(CurveEditorTransformTool::EdgeHighlightAlpha) : FLinearColor::Transparent;

		FGeometry LeftSidebarGeometry, RightSidebarGeometry, TopSidebarGeometry, BottomSidebarGeometry;
		InTransformWidget.GetSidebarGeometry(InAllottedGeometry, LeftSidebarGeometry, RightSidebarGeometry, TopSidebarGeometry, BottomSidebarGeometry);

		// Left Edge
		FSlateDrawElement::MakeBox(OutDrawElements, InPaintOnLayerId, LeftSidebarGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("WhiteBrush")), ESlateDrawEffect::None, LeftEdgeHighlightColor);
		// Right Edge
		FSlateDrawElement::MakeBox(OutDrawElements, InPaintOnLayerId, RightSidebarGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("WhiteBrush")), ESlateDrawEffect::None, RightEdgeHighlightColor);
		// Top Edge
		FSlateDrawElement::MakeBox(OutDrawElements, InPaintOnLayerId, TopSidebarGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("WhiteBrush")), ESlateDrawEffect::None, TopEdgeHighlightColor);
		// Bottom Edge
		FSlateDrawElement::MakeBox(OutDrawElements, InPaintOnLayerId, BottomSidebarGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("WhiteBrush")), ESlateDrawEffect::None, BottomEdgeHighlightColor);

		// Draw arrow markers if falloff is on
		if (FSlateApplication::Get().GetModifierKeys().IsControlDown())
		{
			// Left/Right sidebar arrows
			if (LeftSidebarGeometry.GetLocalSize().Y > CurveEditorTransformTool::ArrowThreshold)
			{
				{
					const FVector2D ArrowBottom = FVector2D(-1.f * CurveEditorTransformTool::ArrowOffset, LeftSidebarGeometry.GetLocalSize().Y * (1.f - CurveEditorTransformTool::ArrowMargin));
					const FVector2D ArrowTop = FVector2D(-1.f * CurveEditorTransformTool::ArrowOffset, LeftSidebarGeometry.GetLocalSize().Y * CurveEditorTransformTool::ArrowMargin);
					const FVector2D ArrowMiddle = (ArrowBottom + ArrowTop) * .5f;
					const FVector2D TopArrowDiagOffset = FVector2D(-1.f * CurveEditorTransformTool::ArrowDiagOffset, CurveEditorTransformTool::ArrowDiagOffset);
					const FVector2D BottomArrowDiagOffset = FVector2D(-1.f * CurveEditorTransformTool::ArrowDiagOffset, -1.f * CurveEditorTransformTool::ArrowDiagOffset);
					TArray<FVector2D> TopHalfArrow = { ArrowMiddle, ArrowTop, ArrowTop + TopArrowDiagOffset };
					TArray<FVector2D> BottomHalfArrow = { ArrowMiddle, ArrowBottom, ArrowBottom + BottomArrowDiagOffset };
					FSlateDrawElement::MakeLines(OutDrawElements, InPaintOnLayerId, LeftSidebarGeometry.ToPaintGeometry(), TopHalfArrow);
					FSlateDrawElement::MakeLines(OutDrawElements, InPaintOnLayerId, LeftSidebarGeometry.ToPaintGeometry(), BottomHalfArrow);
				}
				{
					const FVector2D ArrowBottom = FVector2D(CurveEditorTransformTool::ArrowOffset + CurveEditorTransformTool::ArrowExtraRightOffset, RightSidebarGeometry.GetLocalSize().Y * (1.f - CurveEditorTransformTool::ArrowMargin));
					const FVector2D ArrowTop = FVector2D(CurveEditorTransformTool::ArrowOffset + CurveEditorTransformTool::ArrowExtraRightOffset, RightSidebarGeometry.GetLocalSize().Y * CurveEditorTransformTool::ArrowMargin);
					const FVector2D ArrowMiddle = (ArrowBottom + ArrowTop) * .5f;
					const FVector2D TopArrowDiagOffset = FVector2D(CurveEditorTransformTool::ArrowDiagOffset, CurveEditorTransformTool::ArrowDiagOffset);
					const FVector2D BottomArrowDiagOffset = FVector2D(CurveEditorTransformTool::ArrowDiagOffset, -1.f * CurveEditorTransformTool::ArrowDiagOffset);
					TArray<FVector2D> TopHalfArrow = { ArrowMiddle, ArrowTop, ArrowTop + TopArrowDiagOffset };
					TArray<FVector2D> BottomHalfArrow = { ArrowMiddle, ArrowBottom, ArrowBottom + BottomArrowDiagOffset };
					FSlateDrawElement::MakeLines(OutDrawElements, InPaintOnLayerId, RightSidebarGeometry.ToPaintGeometry(), TopHalfArrow);
					FSlateDrawElement::MakeLines(OutDrawElements, InPaintOnLayerId, RightSidebarGeometry.ToPaintGeometry(), BottomHalfArrow);
				}
			}

			// Top sidebar arrows
			if (TopSidebarGeometry.GetLocalSize().X > CurveEditorTransformTool::ArrowThreshold)
			{
				const FVector2D ArrowLeft = FVector2D(TopSidebarGeometry.GetLocalSize().X * CurveEditorTransformTool::ArrowMargin * -1.f, -1.f * CurveEditorTransformTool::ArrowOffset - CurveEditorTransformTool::ArrowExtraTopOffset);
				const FVector2D ArrowRight = FVector2D(TopSidebarGeometry.GetLocalSize().X * (1.f - CurveEditorTransformTool::ArrowMargin) * -1.f, -1.f * CurveEditorTransformTool::ArrowOffset - CurveEditorTransformTool::ArrowExtraTopOffset);
				const FVector2D ArrowMiddle = (ArrowLeft + ArrowRight) * .5f;
				const FVector2D LeftArrowDiagOffset = FVector2D( -1.f * CurveEditorTransformTool::ArrowDiagOffset, -1.f * CurveEditorTransformTool::ArrowDiagOffset);
				const FVector2D RightArrowDiagOffset = FVector2D(CurveEditorTransformTool::ArrowDiagOffset, -1.f * CurveEditorTransformTool::ArrowDiagOffset);
				TArray<FVector2D> LeftHalfArrow = { ArrowMiddle, ArrowLeft, ArrowLeft + LeftArrowDiagOffset };
				TArray<FVector2D> RightHalfArrow = { ArrowMiddle, ArrowRight, ArrowRight + RightArrowDiagOffset };
				FSlateDrawElement::MakeLines(OutDrawElements, InPaintOnLayerId, RightSidebarGeometry.ToPaintGeometry(), LeftHalfArrow);
				FSlateDrawElement::MakeLines(OutDrawElements, InPaintOnLayerId, RightSidebarGeometry.ToPaintGeometry(), RightHalfArrow);
			}
		}
	}
	// Draw falloff if on
	if (FSlateApplication::Get().GetModifierKeys().IsControlDown())
	{
		FGeometry TopLeftFalloffGeometry, TopRightFalloffGeometry, LeftFalloffGeometry, RightFalloffGeometry;
		InTransformWidget.GetFalloffGeometry(InAllottedGeometry, FalloffHeight, FalloffWidth, TopLeftFalloffGeometry, TopRightFalloffGeometry, LeftFalloffGeometry, RightFalloffGeometry);

		const bool bTopLeft = (InTransformWidget.DisplayAnchorFlags & ECurveEditorAnchorFlags::FalloffTopLeft)  != ECurveEditorAnchorFlags::None || 
			(InTransformWidget.DisplayAnchorFlags & ECurveEditorAnchorFlags::FalloffTopRight) != ECurveEditorAnchorFlags::None;
		const bool bTopRight = (InTransformWidget.DisplayAnchorFlags & ECurveEditorAnchorFlags::FalloffTopLeft) != ECurveEditorAnchorFlags::None ||
			(InTransformWidget.DisplayAnchorFlags & ECurveEditorAnchorFlags::FalloffTopRight) != ECurveEditorAnchorFlags::None;
		const bool bBottomLeft = (InTransformWidget.DisplayAnchorFlags & ECurveEditorAnchorFlags::FalloffLeft) != ECurveEditorAnchorFlags::None || 
			(InTransformWidget.DisplayAnchorFlags & ECurveEditorAnchorFlags::FalloffRight) != ECurveEditorAnchorFlags::None;
		const bool bBottomRight = (InTransformWidget.DisplayAnchorFlags & ECurveEditorAnchorFlags::FalloffLeft) != ECurveEditorAnchorFlags::None ||
			(InTransformWidget.DisplayAnchorFlags & ECurveEditorAnchorFlags::FalloffRight) != ECurveEditorAnchorFlags::None;

		FLinearColor TopLeftHighlightColor = bTopLeft ? FLinearColor::White.CopyWithNewOpacity(CurveEditorTransformTool::EdgeHighlightAlpha) : FLinearColor::Transparent;
		FLinearColor TopRightHighlightColor = bTopRight ? FLinearColor::White.CopyWithNewOpacity(CurveEditorTransformTool::EdgeHighlightAlpha) : FLinearColor::Transparent;
		FLinearColor BottomLeftHighlightColor = bBottomLeft ? FLinearColor::White.CopyWithNewOpacity(CurveEditorTransformTool::EdgeHighlightAlpha) : FLinearColor::Transparent;
		FLinearColor BottomRightHighlightColor = bBottomRight ? FLinearColor::White.CopyWithNewOpacity(CurveEditorTransformTool::EdgeHighlightAlpha) : FLinearColor::Transparent;

		const float Rotate = FMath::DegreesToRadians(45.f);
		// Top Left (Highlight, Corner Icon)
		FSlateDrawElement::MakeRotatedBox(OutDrawElements, InPaintOnLayerId, TopLeftFalloffGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("WhiteBrush")), ESlateDrawEffect::None, Rotate,
			TOptional<FVector2D>(), FSlateDrawElement::RelativeToElement, TopLeftHighlightColor);
		FSlateDrawElement::MakeRotatedBox(OutDrawElements, InPaintOnLayerId, TopLeftFalloffGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("MarqueeSelection")), ESlateDrawEffect::None, Rotate);
		// Top Right										 
		FSlateDrawElement::MakeRotatedBox(OutDrawElements, InPaintOnLayerId, TopRightFalloffGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("WhiteBrush")), ESlateDrawEffect::None, Rotate,
			TOptional<FVector2D>(), FSlateDrawElement::RelativeToElement, TopRightHighlightColor);
		FSlateDrawElement::MakeRotatedBox(OutDrawElements, InPaintOnLayerId, TopRightFalloffGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("MarqueeSelection")), ESlateDrawEffect::None, Rotate);
		// Bottom Left										 
		FSlateDrawElement::MakeRotatedBox(OutDrawElements, InPaintOnLayerId, LeftFalloffGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("WhiteBrush")), ESlateDrawEffect::None, Rotate,
			TOptional<FVector2D>(), FSlateDrawElement::RelativeToElement, BottomLeftHighlightColor);
		FSlateDrawElement::MakeRotatedBox(OutDrawElements, InPaintOnLayerId, LeftFalloffGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("MarqueeSelection")), ESlateDrawEffect::None, Rotate);
		// Bottom Right										 
		FSlateDrawElement::MakeRotatedBox(OutDrawElements, InPaintOnLayerId, RightFalloffGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("WhiteBrush")), ESlateDrawEffect::None, Rotate,
			TOptional<FVector2D>(), FSlateDrawElement::RelativeToElement, BottomRightHighlightColor);
		FSlateDrawElement::MakeRotatedBox(OutDrawElements, InPaintOnLayerId, RightFalloffGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("MarqueeSelection")), ESlateDrawEffect::None, Rotate);

		// Draw falloff weights
		FGeometry GradientGeometry;
		InTransformWidget.GetCenterGeometry(InAllottedGeometry, GradientGeometry);

		// Sample weights at 10 points to get an estimation of the falloff strength
		TArray<FSlateGradientStop> GradientStops;
		GradientStops.Reserve(CurveEditorTransformTool::FalloffGradientSampleSize * 2 + 1);
		TArray<float> SampleVals;
		SampleVals.SetNumUninitialized(CurveEditorTransformTool::FalloffGradientSampleSize);
		for (int32 Index = 0; Index < CurveEditorTransformTool::FalloffGradientSampleSize; Index++)
		{
			SampleVals[Index] = .1f * (Index + 1);
		}

		TArray<float> InterpolatedSamples;
		InterpolatedSamples.Reserve(CurveEditorTransformTool::FalloffGradientSampleSize);
		Algo::Transform(SampleVals, InterpolatedSamples, [this] (float Val) { return FMath::Abs(ModifyWeightByInterpType(Val)); });

		// Draw gradients, left half then right half
		for (int32 Index = 0; Index < CurveEditorTransformTool::FalloffGradientSampleSize; Index++)
		{
			FVector2D AnchorPos;
			AnchorPos.X = (SampleVals[Index]) * (GradientGeometry.GetLocalSize().X * 0.5f * (1-FalloffWidth));
			AnchorPos.Y = GradientGeometry.GetLocalSize().Y * 0.5f;

			FLinearColor StopColor = FLinearColor::Gray.CopyWithNewOpacity((FalloffHeight + (1 - FalloffHeight) * InterpolatedSamples[Index]) * CurveEditorTransformTool::MaxFalloffOpacity);

			GradientStops.Emplace(AnchorPos, StopColor);
		}
		for (int32 Index = 0; Index < CurveEditorTransformTool::FalloffGradientSampleSize; Index++)
		{
			FVector2D AnchorPos;
			AnchorPos.X = (((SampleVals[Index] - .1f) * (1 - FalloffWidth)) + (1 + FalloffWidth)) * (GradientGeometry.GetLocalSize().X * 0.5f);
			AnchorPos.Y = GradientGeometry.GetLocalSize().Y;

			FLinearColor StopColor = FLinearColor::Gray.CopyWithNewOpacity(
				(FalloffHeight + (1 - FalloffHeight) * InterpolatedSamples[CurveEditorTransformTool::FalloffGradientSampleSize - Index - 1]) * CurveEditorTransformTool::MaxFalloffOpacity);

			GradientStops.Emplace(AnchorPos, StopColor);
		}
		
		FSlateDrawElement::MakeGradient(OutDrawElements, InPaintOnLayerId, GradientGeometry.ToPaintGeometry(), GradientStops, Orient_Vertical);
	}
	// draw center marker if on
	if (CurveEditor->GetSelection().Count() != 1)
	{
		FGeometry ScaleCenterGeometry;
		InTransformWidget.GetScaleCenterGeometry(InAllottedGeometry, DisplayRelativeScaleCenter, ScaleCenterGeometry);
		
		const bool bScaleCenter = (InTransformWidget.DisplayAnchorFlags & ECurveEditorAnchorFlags::ScaleCenter) != ECurveEditorAnchorFlags::None;

		FLinearColor ScaleCenterHighlightColor = bScaleCenter ? FLinearColor::White.CopyWithNewOpacity(CurveEditorTransformTool::EdgeHighlightAlpha) : FLinearColor::Transparent;
		const float HighlightThickness = 7.f;

		// draw center scale icon lines
		const double TangentMul = 2.f;
		FVector2D CircleOffset = FVector2D(CurveEditorTransformTool::ScaleCenterRadius, CurveEditorTransformTool::ScaleCenterRadius);
		FSlateDrawElement::MakeSpline(OutDrawElements, InPaintOnLayerId, ScaleCenterGeometry.ToPaintGeometry(), FVector2D(-CurveEditorTransformTool::ScaleCenterRadius, 0.f) + CircleOffset, FVector2D(CurveEditorTransformTool::ScaleCenterRadius, 0.f) * TangentMul,
			FVector2D(0.f, -CurveEditorTransformTool::ScaleCenterRadius) + CircleOffset, FVector2D(0.f, -CurveEditorTransformTool::ScaleCenterRadius) * TangentMul, 1.f);
		FSlateDrawElement::MakeSpline(OutDrawElements, InPaintOnLayerId, ScaleCenterGeometry.ToPaintGeometry(), FVector2D(CurveEditorTransformTool::ScaleCenterRadius, 0.f) + CircleOffset, FVector2D(-CurveEditorTransformTool::ScaleCenterRadius, 0.f) * TangentMul,
			FVector2D(0.f, CurveEditorTransformTool::ScaleCenterRadius) + CircleOffset, FVector2D(0.f, CurveEditorTransformTool::ScaleCenterRadius) * TangentMul, 1.f);
		FSlateDrawElement::MakeSpline(OutDrawElements, InPaintOnLayerId, ScaleCenterGeometry.ToPaintGeometry(), FVector2D(0.f, CurveEditorTransformTool::ScaleCenterRadius) + CircleOffset, FVector2D(0.f, -CurveEditorTransformTool::ScaleCenterRadius) * TangentMul,
			FVector2D(-CurveEditorTransformTool::ScaleCenterRadius, 0.f) + CircleOffset, FVector2D(-CurveEditorTransformTool::ScaleCenterRadius, 0.f) * TangentMul, 1.f);
		FSlateDrawElement::MakeSpline(OutDrawElements, InPaintOnLayerId, ScaleCenterGeometry.ToPaintGeometry(), FVector2D(0.f, -CurveEditorTransformTool::ScaleCenterRadius) + CircleOffset, FVector2D(0.f, CurveEditorTransformTool::ScaleCenterRadius) * TangentMul,
			FVector2D(CurveEditorTransformTool::ScaleCenterRadius, 0.f) + CircleOffset, FVector2D(CurveEditorTransformTool::ScaleCenterRadius, 0.f) * TangentMul, 1.f);

		// draw center scale icon highlight
		FSlateDrawElement::MakeSpline(OutDrawElements, InPaintOnLayerId, ScaleCenterGeometry.ToPaintGeometry(), FVector2D(-CurveEditorTransformTool::ScaleCenterRadius, 0.f) + CircleOffset, FVector2D(CurveEditorTransformTool::ScaleCenterRadius, 0.f) * TangentMul,
			FVector2D(0.f, -CurveEditorTransformTool::ScaleCenterRadius) + CircleOffset, FVector2D(0.f, -CurveEditorTransformTool::ScaleCenterRadius) * TangentMul, HighlightThickness, ESlateDrawEffect::None, ScaleCenterHighlightColor);
		FSlateDrawElement::MakeSpline(OutDrawElements, InPaintOnLayerId, ScaleCenterGeometry.ToPaintGeometry(), FVector2D(CurveEditorTransformTool::ScaleCenterRadius, 0.f) + CircleOffset, FVector2D(-CurveEditorTransformTool::ScaleCenterRadius, 0.f) * TangentMul,
			FVector2D(0.f, CurveEditorTransformTool::ScaleCenterRadius) + CircleOffset, FVector2D(0.f, CurveEditorTransformTool::ScaleCenterRadius) * TangentMul, HighlightThickness, ESlateDrawEffect::None, ScaleCenterHighlightColor);
		FSlateDrawElement::MakeSpline(OutDrawElements, InPaintOnLayerId, ScaleCenterGeometry.ToPaintGeometry(), FVector2D(0.f, CurveEditorTransformTool::ScaleCenterRadius) + CircleOffset, FVector2D(0.f, -CurveEditorTransformTool::ScaleCenterRadius) * TangentMul,
			FVector2D(-CurveEditorTransformTool::ScaleCenterRadius, 0.f) + CircleOffset, FVector2D(-CurveEditorTransformTool::ScaleCenterRadius, 0.f) * TangentMul, HighlightThickness, ESlateDrawEffect::None, ScaleCenterHighlightColor);
		FSlateDrawElement::MakeSpline(OutDrawElements, InPaintOnLayerId, ScaleCenterGeometry.ToPaintGeometry(), FVector2D(0.f, -CurveEditorTransformTool::ScaleCenterRadius) + CircleOffset, FVector2D(0.f, CurveEditorTransformTool::ScaleCenterRadius) * TangentMul,
			FVector2D(CurveEditorTransformTool::ScaleCenterRadius, 0.f) + CircleOffset, FVector2D(CurveEditorTransformTool::ScaleCenterRadius, 0.f) * TangentMul, HighlightThickness, ESlateDrawEffect::None, ScaleCenterHighlightColor);

		if (bScaleCenter)
		{
			FGeometry CenterIndicatorGeometry;
			InTransformWidget.GetCenterIndicatorGeometry(InAllottedGeometry, CenterIndicatorGeometry);
			FLinearColor CenterIndicatorHighlightColor = FLinearColor::White.CopyWithNewOpacity(CurveEditorTransformTool::EdgeHighlightAlpha);

			FSlateDrawElement::MakeBox(OutDrawElements, InPaintOnLayerId, CenterIndicatorGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("WhiteBrush")), ESlateDrawEffect::None, CenterIndicatorHighlightColor);
		}
	}

	// Draw the four corners + highlights
	{
		FGeometry TopLeftCornerGeometry, TopRightCornerGeometry, BottomLeftCornerGeometry, BottomRightCornerGeometry;
		InTransformWidget.GetCornerGeometry(InAllottedGeometry, TopLeftCornerGeometry, TopRightCornerGeometry, BottomLeftCornerGeometry, BottomRightCornerGeometry);

		const bool bTopLeft =		InTransformWidget.DisplayAnchorFlags == (ECurveEditorAnchorFlags::Top		| ECurveEditorAnchorFlags::Left );
		const bool bTopRight =		InTransformWidget.DisplayAnchorFlags == (ECurveEditorAnchorFlags::Top		| ECurveEditorAnchorFlags::Right);
		const bool bBottomLeft =	InTransformWidget.DisplayAnchorFlags == (ECurveEditorAnchorFlags::Bottom	| ECurveEditorAnchorFlags::Left );
		const bool bBottomRight =	InTransformWidget.DisplayAnchorFlags == (ECurveEditorAnchorFlags::Bottom	| ECurveEditorAnchorFlags::Right);

		FLinearColor TopLeftHighlightColor		= bTopLeft		? FLinearColor::White.CopyWithNewOpacity(CurveEditorTransformTool::EdgeHighlightAlpha) : FLinearColor::Transparent;
		FLinearColor TopRightHighlightColor		= bTopRight		? FLinearColor::White.CopyWithNewOpacity(CurveEditorTransformTool::EdgeHighlightAlpha) : FLinearColor::Transparent;
		FLinearColor BottomLeftHighlightColor	= bBottomLeft	? FLinearColor::White.CopyWithNewOpacity(CurveEditorTransformTool::EdgeHighlightAlpha) : FLinearColor::Transparent;
		FLinearColor BottomRightHighlightColor	= bBottomRight  ? FLinearColor::White.CopyWithNewOpacity(CurveEditorTransformTool::EdgeHighlightAlpha) : FLinearColor::Transparent;

		// Top Left (Highlight, Corner Icon)
		FSlateDrawElement::MakeBox(OutDrawElements, InPaintOnLayerId, TopLeftCornerGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("WhiteBrush")), ESlateDrawEffect::None, TopLeftHighlightColor);
		FSlateDrawElement::MakeBox(OutDrawElements, InPaintOnLayerId, TopLeftCornerGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("MarqueeSelection")));
		// Top Right										 
		FSlateDrawElement::MakeBox(OutDrawElements, InPaintOnLayerId, TopRightCornerGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("WhiteBrush")), ESlateDrawEffect::None, TopRightHighlightColor);
		FSlateDrawElement::MakeBox(OutDrawElements, InPaintOnLayerId, TopRightCornerGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("MarqueeSelection")));
		// Bottom Left										 
		FSlateDrawElement::MakeBox(OutDrawElements, InPaintOnLayerId, BottomLeftCornerGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("WhiteBrush")), ESlateDrawEffect::None, BottomLeftHighlightColor);
		FSlateDrawElement::MakeBox(OutDrawElements, InPaintOnLayerId, BottomLeftCornerGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("MarqueeSelection")));
		// Bottom Right										 
		FSlateDrawElement::MakeBox(OutDrawElements, InPaintOnLayerId, BottomRightCornerGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("WhiteBrush")), ESlateDrawEffect::None, BottomRightHighlightColor);
		FSlateDrawElement::MakeBox(OutDrawElements, InPaintOnLayerId, BottomRightCornerGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("MarqueeSelection")));
	}
}


void FCurveEditorTransformTool::OnDragStart()
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return;
	}

	using namespace UE::CurveEditor;
	ActiveTransaction = MakeUnique<FScopedCurveChange>(
		FCurvesSnapshotBuilder(CurveEditor,
			ECurveChangeFlags::KeyAttributes | ECurveChangeFlags::MoveKeysAndRemoveStackedKeys
			).TrackSelectedCurves(),
		LOCTEXT("CurveEditorTransformToolTransaction", "Transform Key(s)")
		);
	TangentFlattener.Emplace(*CurveEditor);

	CurveEditor->SuppressBoundTransformUpdates(true);

	// We need to cache our key data because all of our calculations have to be relative to the starting data and not the current per-frame data.
	KeysByCurve.Reset();
	bool bMinMaxIsSet = false; // will be better cached than TOptional and we only need to check this flag here.

	for (const TTuple<FCurveModelID, FKeyHandleSet>& Pair : CurveEditor->GetSelection().GetAll())
	{
		FCurveModelID CurveID = Pair.Key;
		FCurveModel*  Curve = CurveEditor->FindCurve(CurveID);

		if (ensureAlways(Curve))
		{
			TArrayView<const FKeyHandle> Handles = Pair.Value.AsArray();

			FKeyData& KeyData = KeysByCurve.Emplace_GetRef(CurveID);
			KeyData.Handles = TArray<FKeyHandle>(Handles.GetData(), Handles.Num());

			KeyData.StartKeyPositions.SetNumZeroed(KeyData.Handles.Num());
			Curve->GetKeyPositions(KeyData.Handles, KeyData.StartKeyPositions);
			KeyData.LastDraggedKeyPositions = KeyData.StartKeyPositions;
			for (const FKeyPosition& KeyPosition : KeyData.StartKeyPositions)
			{
				if (bMinMaxIsSet)
				{
					if (KeyPosition.InputValue < InputMinMax.GetLowerBoundValue())
					{
						InputMinMax.SetLowerBoundValue(KeyPosition.InputValue);
					}
					else if (KeyPosition.InputValue > InputMinMax.GetUpperBoundValue())
					{
						InputMinMax.SetUpperBoundValue(KeyPosition.InputValue);
					}
				}
				else
				{
					bMinMaxIsSet = true;
					InputMinMax = TRange<double>(KeyPosition.InputValue);
				}
			}
		}
	}

	TransformWidget.StartSize = TransformWidget.Size;
	TransformWidget.StartPosition = TransformWidget.Position;
	SnappingState.Reset();
}

void FCurveEditorTransformTool::OnDrag(const FPointerEvent& InMouseEvent, const FVector2D& InLocalMousePosition)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return;
	}

	TArray<FKeyPosition> NewKeyPositionScratch;

	FSlateLayoutTransform ContainerToAbsolute = CurveEditor->GetPanel()->GetViewContainerGeometry().GetAccumulatedLayoutTransform().Inverse();
	const bool bFalloffOn = InMouseEvent.IsControlDown();
	if (bFalloffOn && (TransformWidget.SelectedAnchorFlags & ECurveEditorAnchorFlags::FalloffAll) != ECurveEditorAnchorFlags::None)
	{
		const bool bTopLeft = ((TransformWidget.SelectedAnchorFlags & ECurveEditorAnchorFlags::FalloffTopLeft) != ECurveEditorAnchorFlags::None);
		const bool bTopRight = ((TransformWidget.SelectedAnchorFlags & ECurveEditorAnchorFlags::FalloffTopRight) != ECurveEditorAnchorFlags::None);
		const bool bLeft = ((TransformWidget.SelectedAnchorFlags & ECurveEditorAnchorFlags::FalloffLeft) != ECurveEditorAnchorFlags::None);
		const bool bRight = ((TransformWidget.SelectedAnchorFlags & ECurveEditorAnchorFlags::FalloffRight) != ECurveEditorAnchorFlags::None);
		if ((bLeft || bRight) && TransformWidget.Size.Y > SMALL_NUMBER)
		{
			FVector2D MouseDelta = InLocalMousePosition - TransformWidget.Position;
			FalloffHeight = 1.0f - MouseDelta.Y / TransformWidget.Size.Y;
			FalloffHeight = FMath::Clamp(FalloffHeight, 0.0f, 1.0f);
		}
		if ((bTopLeft || bTopRight) && TransformWidget.Size.X > SMALL_NUMBER)
		{
			FVector2D MouseDelta = InLocalMousePosition - TransformWidget.Position;

			FalloffWidth = (MouseDelta.X) / TransformWidget.Size.X;
			FalloffWidth = FMath::Clamp(FalloffWidth, 0.0f, 1.0f);
			if (FalloffWidth < 0.5f)
			{
				FalloffWidth = (0.5 - FalloffWidth) * 2.0f;
			}
			else if (FalloffWidth > 0.5f)
			{
				FalloffWidth = (FalloffWidth - 0.5f) * 2.0f;
			}
		}
	}
	else if ((TransformWidget.SelectedAnchorFlags & ECurveEditorAnchorFlags::ScaleCenter) != ECurveEditorAnchorFlags::None && CurveEditor->GetSelection().Count() != 1)
	{
		const FVector2D MouseDelta = InLocalMousePosition - TransformWidget.BoundsPosition;
		RelativeScaleCenter = MouseDelta / TransformWidget.BoundsSize;

		const float SnapThreshold = 20.f;

		FVector2D ClosestInRange = InLocalMousePosition;
		double ClosestInRangeDist = TNumericLimits<double>::Max();

		// Snap to keys
		{
			for (const TPair<FCurveModelID, FKeyHandleSet>& CurveKeys : CurveEditor->GetSelection().GetAll())
			{
				TArray<FKeyPosition> KeyPositions;
				KeyPositions.SetNumUninitialized(CurveKeys.Value.Num());
				CurveEditor->GetCurves()[CurveKeys.Key]->GetKeyPositions(CurveKeys.Value.AsArray(), KeyPositions);

				const SCurveEditorView* View = CurveEditor->FindFirstInteractiveView(CurveKeys.Key);
				if (!View)
				{
					continue;
				}

				FCurveEditorScreenSpace CurveSpace = View->GetCurveSpace(CurveKeys.Key);

				for (const FKeyPosition& KeyPos : KeyPositions)
				{
					const FVector2D ViewSpacePosition = FVector2D(CurveSpace.SecondsToScreen(KeyPos.InputValue), CurveSpace.ValueToScreen(KeyPos.OutputValue));
					const double KeyDistance = FVector2D::Distance(ViewSpacePosition, InLocalMousePosition);
					if (KeyDistance < ClosestInRangeDist)
					{
						ClosestInRangeDist = KeyDistance;
						ClosestInRange = ViewSpacePosition;
					}
				}
			}

			// Not snapped, reset everything
			if (ClosestInRangeDist > SnapThreshold)
			{
				ClosestInRange = InLocalMousePosition;
				ClosestInRangeDist = TNumericLimits<double>::Max();
			}
		}

		// Snap to grid
		if (bCurvesHaveSameScales && (CurveEditor->IsInputSnappingEnabled() || CurveEditor->IsOutputSnappingEnabled()))
		{
			// Snap the mouse to the grid according to the first curve's snap metrics (assuming all curves have the same view scales)
			const FCurveModelID FirstCurveID = (*CurveEditor->GetSelection().GetAll().CreateConstIterator()).Key;

			const SCurveEditorView* View = CurveEditor->FindFirstInteractiveView(FirstCurveID);
			if (View)
			{
				FCurveEditorScreenSpace CurveSpace = View->GetCurveSpace(FirstCurveID);

				FCurveSnapMetrics SnapMetrics = CurveEditor->GetCurveSnapMetrics(FirstCurveID);
				const FVector2D CurveSpaceSnappedMousePoint = FVector2D(SnapMetrics.SnapInputSeconds(CurveSpace.ScreenToSeconds(InLocalMousePosition.X)), SnapMetrics.SnapOutput(CurveSpace.ScreenToValue(InLocalMousePosition.Y)));
				const FVector2D ViewSpaceSnappedMousePoint = FVector2D(CurveSpace.SecondsToScreen(CurveSpaceSnappedMousePoint.X), CurveSpace.ValueToScreen(CurveSpaceSnappedMousePoint.Y));

				float Dist = 0.f;
				if ((Dist = FVector2D::Distance(ViewSpaceSnappedMousePoint, InLocalMousePosition)) < ClosestInRangeDist)
				{
					ClosestInRange = ViewSpaceSnappedMousePoint;
					ClosestInRangeDist = Dist;
				}
			}
		}

		// Snap to edges or corners
		{
			float Dist = FMath::Abs(InLocalMousePosition.Y - (TransformWidget.Position.Y));
			if (Dist < FMath::Min<float>(SnapThreshold, ClosestInRangeDist))
			{
				ClosestInRange.Y = TransformWidget.Position.Y;
				ClosestInRangeDist = Dist;
			}

			Dist = FMath::Abs(InLocalMousePosition.Y - (TransformWidget.Position.Y + TransformWidget.Size.Y));
			if (Dist < FMath::Min<float>(SnapThreshold, ClosestInRangeDist))
			{
				ClosestInRange.Y = TransformWidget.Position.Y + TransformWidget.Size.Y;
				ClosestInRangeDist = Dist;
			}

			Dist = FMath::Abs(InLocalMousePosition.X - (TransformWidget.Position.X));
			if (Dist < FMath::Min<float>(SnapThreshold, ClosestInRangeDist))
			{
				ClosestInRange.X = TransformWidget.Position.X;
				ClosestInRangeDist = Dist;
			}

			Dist = FMath::Abs(InLocalMousePosition.X - (TransformWidget.Position.X + TransformWidget.Size.X));
			if (Dist < FMath::Min<float>(SnapThreshold, ClosestInRangeDist))
			{
				ClosestInRange.X = TransformWidget.Position.X + TransformWidget.Size.X;
				ClosestInRangeDist = Dist;
			}
		}

		// Snap to center indicator
		{
			float Dist = 0.f;
			if ((Dist = FVector2D::Distance(InLocalMousePosition, TransformWidget.BoundsPosition + TransformWidget.BoundsSize * 0.5f)) < ClosestInRangeDist)
			{
				ClosestInRange = TransformWidget.BoundsPosition + TransformWidget.BoundsSize * 0.5f;
				ClosestInRangeDist = Dist;
			}
		}

		if (ClosestInRangeDist < SnapThreshold)
		{
			const FVector2D ViewSpaceDelta = ClosestInRange - TransformWidget.BoundsPosition;
			RelativeScaleCenter = ViewSpaceDelta / TransformWidget.BoundsSize;
		}

		DisplayRelativeScaleCenter = RelativeScaleCenter;
	}
	else
	{
		// Dragging the center is the easy case!
		if (TransformWidget.SelectedAnchorFlags == ECurveEditorAnchorFlags::Center)
		{
			const FVector2D AxisLockedMousePosition = CurveEditor->GetAxisSnap().GetSnappedPosition(InitialMousePosition, LastMousePosition, InMouseEvent.GetScreenSpacePosition(), InMouseEvent, SnappingState);
			{
				FVector2D MouseDelta = AxisLockedMousePosition - InitialMousePosition;
				TransformWidget.Position = TransformWidget.StartPosition + MouseDelta;
			}

			for (FKeyData& KeyData : KeysByCurve)
			{
				const SCurveEditorView* View = CurveEditor->FindFirstInteractiveView(KeyData.CurveID);
				if (!View)
				{
					continue;
				}

				FCurveModel* CurveModel = CurveEditor->FindCurve(KeyData.CurveID);
				check(CurveModel);

				FCurveEditorScreenSpace CurveSpace = View->GetCurveSpace(KeyData.CurveID);

				const double DeltaInput = (AxisLockedMousePosition.X - InitialMousePosition.X) / CurveSpace.PixelsPerInput();
				const double DeltaOutput = -(AxisLockedMousePosition.Y - InitialMousePosition.Y) / CurveSpace.PixelsPerOutput();

				NewKeyPositionScratch.Reset();
				NewKeyPositionScratch.Reserve(KeyData.StartKeyPositions.Num());

				for (FKeyPosition StartPosition : KeyData.StartKeyPositions)
				{
					StartPosition.InputValue += DeltaInput;
					StartPosition.OutputValue += DeltaOutput;

					StartPosition.InputValue = View->IsTimeSnapEnabled() ? CurveEditor->GetCurveSnapMetrics(KeyData.CurveID).SnapInputSeconds(StartPosition.InputValue) : StartPosition.InputValue;
					StartPosition.OutputValue = View->IsValueSnapEnabled() ? CurveEditor->GetCurveSnapMetrics(KeyData.CurveID).SnapOutput(StartPosition.OutputValue) : StartPosition.OutputValue;

					NewKeyPositionScratch.Add(StartPosition);
				}

				CurveModel->SetKeyPositions(KeyData.Handles, NewKeyPositionScratch, EPropertyChangeType::Interactive);
				KeyData.LastDraggedKeyPositions = NewKeyPositionScratch;
			}
		}
		else if (TransformWidget.SelectedAnchorFlags != ECurveEditorAnchorFlags::None)
		{
			const bool bAffectsX = ((TransformWidget.SelectedAnchorFlags & ECurveEditorAnchorFlags::Left) != ECurveEditorAnchorFlags::None ||
				(TransformWidget.SelectedAnchorFlags & ECurveEditorAnchorFlags::Right) != ECurveEditorAnchorFlags::None);

			const bool bAffectsY = ((TransformWidget.SelectedAnchorFlags & ECurveEditorAnchorFlags::Top) != ECurveEditorAnchorFlags::None ||
				(TransformWidget.SelectedAnchorFlags & ECurveEditorAnchorFlags::Bottom) != ECurveEditorAnchorFlags::None);

			// This is the absolute change since our KeysByCurve was initialized.
			const FVector2D MouseDelta = InMouseEvent.GetScreenSpacePosition() - InitialMousePosition;

			// We have to flip the delta depending on which edge you grabbed so that the change always goes towards the mouse.
			const float InputMulSign = (TransformWidget.SelectedAnchorFlags & ECurveEditorAnchorFlags::Left) != ECurveEditorAnchorFlags::None ? -1.0f : 1.0f;
			const float OutputMulSign = (TransformWidget.SelectedAnchorFlags & ECurveEditorAnchorFlags::Top) != ECurveEditorAnchorFlags::None ? -1.0f : 1.0f;
			const FVector2D AxisFixedMouseDelta = FVector2D(MouseDelta.X * InputMulSign, MouseDelta.Y * OutputMulSign);
			const FVector2D PanelSpaceCenter = TransformWidget.StartPosition + (TransformWidget.StartSize * RelativeScaleCenter);
			const FVector2D ChangeAmount = (AxisFixedMouseDelta / TransformWidget.StartSize);

			ScaleFrom(PanelSpaceCenter, ChangeAmount, bFalloffOn, bAffectsX, bAffectsY);

			TransformWidget.DisplayAnchorFlags = TransformWidget.SelectedAnchorFlags;
			DisplayRelativeScaleCenter = RelativeScaleCenter;
			// If x scale flips, toggle left/right flags
			if (ChangeAmount.X < -1.f)
			{
				TransformWidget.DisplayAnchorFlags ^= ECurveEditorAnchorFlags::Left;
				TransformWidget.DisplayAnchorFlags ^= ECurveEditorAnchorFlags::Right;
				DisplayRelativeScaleCenter.X = 1 - RelativeScaleCenter.X;
			}
			// If y scale flips, toggle top/bottom flags
			if (ChangeAmount.Y < -1.f)
			{
				TransformWidget.DisplayAnchorFlags ^= ECurveEditorAnchorFlags::Top;
				TransformWidget.DisplayAnchorFlags ^= ECurveEditorAnchorFlags::Bottom;
				DisplayRelativeScaleCenter.Y = 1 - RelativeScaleCenter.Y;
			}
		}
	}
	LastMousePosition = InMouseEvent.GetScreenSpacePosition();
	UpdateMarqueeBoundingBox();

	UpdateToolOptions();
}

void FCurveEditorTransformTool::OnDragEnd()
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor.IsValid())
	{
		return;
	}

	for (const FKeyData& KeyData : KeysByCurve)
	{
		if (FCurveModel* Curve = CurveEditor->FindCurve(KeyData.CurveID))
		{
			Curve->SetKeyPositions(KeyData.Handles, KeyData.LastDraggedKeyPositions, EPropertyChangeType::ValueSet);
		}
	}

	CurveEditor->SuppressBoundTransformUpdates(false);

	TangentFlattener.Reset();
	// This finalizes the transaction
	ActiveTransaction.Reset();

	UpdateToolOptions();
}

void FCurveEditorTransformTool::StopDragIfPossible()
{
	if (DelayedDrag.IsSet())
	{
		if (DelayedDrag->IsDragging())
		{
			OnDragEnd();
		}

		DelayedDrag.Reset();
	}
}

void FCurveEditorTransformTool::ScaleFrom(const FVector2D& InPanelSpaceCenter, const FVector2D& InChangeAmount, const bool bInFalloffOn, const bool bInAffectsX, const bool bInAffectsY) 
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return;
	}

	// Rescaling is where things get tricky. If a user is selecting an edge we scale on one axis at once, and if they select a corner we scale on two axis at once.
	// If they press alt, we scale relative to the center (instead of relative to the opposite corner). Because these are all bit-flag'd together we can implement
	// this generically by scaling coordinates relative to an arbitrary center

	FVector2D PercentChanged = FVector2D(1.0f, 1.0f) + InChangeAmount; // ie: 5 pixel change on a 100 wide gives you 1.05

	FSlateLayoutTransform ContainerToAbsolute = CurveEditor->GetPanel()->GetViewContainerGeometry().GetAccumulatedLayoutTransform().Inverse();

	TArray<FKeyPosition> NewKeyPositionScratch;

	// We now know if we need to affect both X and Y, and we know where we're scaling from. Now we can loop through the keys and actually modify their positions.
	// We perform the scale on both axis (for simplicity) and then read which axis it should effect before assigning it back to the key position.
	for (FKeyData& KeyData : KeysByCurve)
	{
		const SCurveEditorView* View = CurveEditor->FindFirstInteractiveView(KeyData.CurveID);
		if (!View)
		{
			continue;
		}

		FCurveModel* CurveModel = CurveEditor->FindCurve(KeyData.CurveID);
		check(CurveModel);

		// Compute the curve-space center for transformation by transforming the center from panel space to view space, then to curve space
		FSlateLayoutTransform OuterToInnerTransform = Concatenate(ContainerToAbsolute, View->GetCachedGeometry().GetAccumulatedLayoutTransform()).Inverse();
		FVector2D ViewSpaceCenter = OuterToInnerTransform.TransformPoint(InPanelSpaceCenter);

		FCurveEditorScreenSpace CurveSpace = View->GetCurveSpace(KeyData.CurveID);
		const double CurveSpaceCenterInput = CurveSpace.ScreenToSeconds(ViewSpaceCenter.X);
		const double CurveSpaceCenterOutput = CurveSpace.ScreenToValue(ViewSpaceCenter.Y);

		NewKeyPositionScratch.Reset();
		NewKeyPositionScratch.Reserve(KeyData.StartKeyPositions.Num());

		FCurveSnapMetrics SnapMetrics = CurveEditor->GetCurveSnapMetrics(KeyData.CurveID);

		for (FKeyPosition StartPosition : KeyData.StartKeyPositions)
		{
			// Step 1 is to modify the Y percent changed if falloff is on
			if (bInFalloffOn)
			{
				const double FalloffScale = GetFalloffWeight(StartPosition.InputValue);
				PercentChanged.Y = 1.0f + InChangeAmount.Y * FalloffScale;
			}

			// Step 2 is to rescale the position of the key by the percentage change on each axis.
			const double ScaledInput = (StartPosition.InputValue - CurveSpaceCenterInput) * PercentChanged.X; // *InputMulSign;
			const double ScaledOutput = (StartPosition.OutputValue - CurveSpaceCenterOutput) * PercentChanged.Y; // *OutputMulSign;

			// Step 3 is to subtract it from the center position so we support scaling from places other than zero
			const double NewInput = CurveSpaceCenterInput + ScaledInput;
			const double NewOutput = CurveSpaceCenterOutput + ScaledOutput;

			// Snap the new values to the grid. We calculate both X and Y changes for ease of programming above and just limit which one it applies to.
			// This includes snapping, otherwise dragging on an edge can cause it to snap on the opposite axis.
			if (bInAffectsX)
			{
				StartPosition.InputValue = View->IsTimeSnapEnabled() ? SnapMetrics.SnapInputSeconds(NewInput) : NewInput;
			}
			if (bInAffectsY)
			{
				StartPosition.OutputValue = View->IsValueSnapEnabled() ? SnapMetrics.SnapOutput(NewOutput) : NewOutput;
			}

			NewKeyPositionScratch.Add(StartPosition);
		}

		CurveModel->SetKeyPositions(KeyData.Handles, NewKeyPositionScratch, EPropertyChangeType::Interactive);
		KeyData.LastDraggedKeyPositions = NewKeyPositionScratch;

	}

	// This recalculates the user specified tangents based on the distance moved. We'll flatten the curves depending on the resulting selection's height.
	if (bInAffectsY)
	{
		const bool bIsMirroring = PercentChanged.Y < 0.0;
		TangentFlattener->ComputeMirroringParallel(*CurveEditor, bIsMirroring);
	}
}

namespace UE::CurveEditorTools
{
static bool HaveSameScales(const FCurveEditor& CurveEditor)
{
	TArray<FCurveModelID> CurveModelIDs;
	CurveEditor.GetSelection().GetAll().GetKeys(CurveModelIDs);
	
	if (CurveModelIDs.Num() == 0)
	{
		return false;
	}

	if (CurveModelIDs.Num() == 1)
	{
		return true;
	}

	// To handle FindFirstInteractiveView returning nullptr, we'll save the first FCurveEditorScreenSpace we encounter and compare against the rest.
	bool bCurvesHaveSameScales = true;
	TOptional<FCurveEditorScreenSpace> FirstCurveViewSpace;
	
	for (int32 ModelIndex = 0; ModelIndex < CurveModelIDs.Num(); ModelIndex++)
	{
		const SCurveEditorView* View = CurveEditor.FindFirstInteractiveView(CurveModelIDs[ModelIndex]);
		if (!ensureMsgf(View, TEXT("No view found for curve. Probably the selection contains stale keys.")))
		{
			continue;
		}
		
		const FCurveEditorScreenSpace CurveViewSpace = View->GetCurveSpace(CurveModelIDs[ModelIndex]);
		if (FirstCurveViewSpace.IsSet())
		{
			bCurvesHaveSameScales &= FMath::IsNearlyEqual(CurveViewSpace.PixelsPerInput(), FirstCurveViewSpace->PixelsPerInput())
				&& FMath::IsNearlyEqual(CurveViewSpace.PixelsPerOutput(), FirstCurveViewSpace->PixelsPerOutput());
		}
		else
		{
			FirstCurveViewSpace = CurveViewSpace;
		}
		
	}

	return bCurvesHaveSameScales;
}
}

void FCurveEditorTransformTool::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// We update the size and position of the box every frame as some scale operations aren't 1:1
	// so this keeps the box visually containing all keys even if the mouse position no longer quite 
	// matches up.
	UpdateMarqueeBoundingBox();

	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return;
	}

	const FFrameRate DisplayRate = CurveEditor->GetTimeSliderController() ? CurveEditor->GetTimeSliderController()->GetDisplayRate() : FFrameRate();
	const FFrameRate TickResolution = CurveEditor->GetTimeSliderController() ? CurveEditor->GetTimeSliderController()->GetTickResolution() : FFrameRate();

	// Check if all curve models have the same scales
	bCurvesHaveSameScales = UE::CurveEditorTools::HaveSameScales(*CurveEditor);

	// remove tool options widget if less than two selected or there are two different curves
	if ((CurveEditor->GetSelection().Count() <= 1 || !bCurvesHaveSameScales) 
		&& ToolOptionsOnScope != nullptr)
	{
		ToolOptionsOnScope = nullptr;
		OnOptionsRefreshDelegate.Broadcast();
	}
	else if ((CurveEditor->GetSelection().Count() > 1 && bCurvesHaveSameScales) 
		&& ToolOptionsOnScope == nullptr)
	{
		ToolOptionsOnScope = MakeShared<FStructOnScope>(FTransformToolOptions::StaticStruct(), (uint8*)&ToolOptions);
		OnOptionsRefreshDelegate.Broadcast();
	}

	// check if there have been any changes made outside of the tool
	if (bCurvesHaveSameScales)
	{
		const FCurveModelID FirstCurveID = (*CurveEditor->GetSelection().GetAll().CreateConstIterator()).Key;

		const SCurveEditorView* View = CurveEditor->FindFirstInteractiveView(FirstCurveID);
		if (!View)
		{
			return;
		}

		FCurveEditorScreenSpace CurveSpace = View->GetCurveSpace(FirstCurveID);

		const double RightBoundSeconds = DisplayRate.AsSeconds(FFrameRate::TransformTime(ToolOptions.RightBound, TickResolution, DisplayRate));
		const double LeftBoundSeconds = DisplayRate.AsSeconds(FFrameRate::TransformTime(ToolOptions.LeftBound, TickResolution, DisplayRate));

		const FVector2D OptionsSize = FVector2D(RightBoundSeconds - LeftBoundSeconds, ToolOptions.LowerBound - ToolOptions.UpperBound);
		const FVector2D ViewSpaceOptionsSize = FVector2D(CurveSpace.PixelsPerInput() * FMath::Abs(OptionsSize.X), CurveSpace.PixelsPerOutput() * FMath::Abs(OptionsSize.Y));
		if ((!DelayedDrag || (DelayedDrag && !DelayedDrag->IsDragging())) && 
			((TransformWidget.BoundsSize - ViewSpaceOptionsSize).IsNearlyZero() || SelectionSerialNumber != CurveEditor->GetSelection().GetSerialNumber()))
		{
			UpdateToolOptions();

			SelectionSerialNumber = CurveEditor->GetSelection().GetSerialNumber();
		}
	}
}

void FCurveEditorTransformTool::BindCommands(TSharedRef<FUICommandList> CommandBindings)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor)
	{
		FIsActionChecked TransformToolIsActive = FIsActionChecked::CreateSP(CurveEditor.ToSharedRef(), &FCurveEditor::IsToolActive, ToolID);
		FExecuteAction ActivateTransformTool = FExecuteAction::CreateSP(CurveEditor.ToSharedRef(), &FCurveEditor::MakeToolActive, ToolID);

		CommandBindings->MapAction(FCurveEditorToolCommands::Get().ActivateTransformTool, ActivateTransformTool, FCanExecuteAction(), TransformToolIsActive);
	}
}

double FCurveEditorTransformTool::GetFalloffWeight(double InputValue) const
{
	const double InputSize = (InputMinMax.GetUpperBoundValue() - InputMinMax.GetLowerBoundValue());
	const double InputMidPoint = InputSize * 0.5;
	const double MidPointDiff = InputMidPoint * FalloffWidth;
	TRange<double> InputMiddleRange(InputMidPoint - MidPointDiff, InputMidPoint + MidPointDiff);
	if (InputValue < InputMiddleRange.GetLowerBoundValue())
	{
		const double LinearScale = (InputValue - InputMinMax.GetLowerBoundValue()) / (InputMiddleRange.GetLowerBoundValue() - InputMinMax.GetLowerBoundValue());
		double  Weight = (LinearScale * (1.0f - FalloffHeight) + FalloffHeight);
		Weight = ModifyWeightByInterpType(Weight);
		return Weight;
	}
	else if (InputValue > InputMiddleRange.GetUpperBoundValue())
	{
		const double  LinearScale = 1.0 - (InputValue - InputMiddleRange.GetUpperBoundValue()) / (InputMinMax.GetUpperBoundValue() - InputMiddleRange.GetUpperBoundValue());
		double  Weight = (LinearScale * (1.0f - FalloffHeight) + FalloffHeight);
		Weight = ModifyWeightByInterpType(Weight);
		return Weight;
	}

	return 1.0;
}

double FCurveEditorTransformTool::ModifyWeightByInterpType(double Value) const
{
	float Result = Value;
	switch (ToolOptions.FalloffInterpType)
	{
		case EToolTransformInterpType::Linear:
		{
			Result = FMath::Clamp<float>(Value, 0.f, 1.f);
			break;
		}
		case EToolTransformInterpType::Sinusoidal:
		{
			Result = FMath::Clamp<float>((FMath::Sin(Value * PI - HALF_PI) + 1.f) / 2.f, 0.f, 1.f);
			break;
		}
		case EToolTransformInterpType::Cubic:
		{
			Result = FMath::Clamp<float>(FMath::CubicInterp<float>(0.f, 0.f, 1.f, 0.f, Value), 0.f, 1.f);
			break;
		}
		case EToolTransformInterpType::CircularIn:
		{
			Result = FMath::Clamp<float>(FMath::InterpCircularIn<float>(0.0f, 1.0f, Value), 0.0f, 1.0f);
			break;
		}
		case EToolTransformInterpType::CircularOut:
		{
			Result = FMath::Clamp<float>(FMath::InterpCircularOut<float>(0.0f, 1.0f, Value), 0.0f, 1.0f);
			break;
		}
		case EToolTransformInterpType::ExpIn:
		{
			Result = FMath::Clamp<float>(FMath::InterpExpoIn<float>(0.0f, 1.0f, Value), 0.0f, 1.0f);
			break;
		}
		case EToolTransformInterpType::ExpOut:
		{
			Result = FMath::Clamp<float>(FMath::InterpExpoOut<float>(0.0f, 1.0f, Value), 0.0f, 1.0f);
			break;
		}
	}
	return Result;
}
#undef LOCTEXT_NAMESPACE // "CurveEditorToolCommands"