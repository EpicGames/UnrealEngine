// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditorTypes.h"
#include "ICurveEditorToolExtension.h"
#include "Framework/DelayedDrag.h"
#include "CurveDataAbstraction.h"
#include "ScopedTransaction.h"
#include "CurveEditorSnapMetrics.h"
#include "Misc/Mirror/TangentSelectionFlattener.h"
#include "Modification/Utils/ScopedCurveChange.h"
#include "CurveEditorTransformTool.generated.h"

class FCurveEditor;
struct FPointerEvent;

enum class ECurveEditorAnchorFlags : uint16
{
	None = 0x00,
	Top = 0x01,
	Left = 0x02,
	Right = 0x04,
	Bottom = 0x08,
	Center = 0x10,
	FalloffTopLeft = 0x20,
	FalloffTopRight = 0x40,
	FalloffLeft = 0x80,
	FalloffRight = 0x100,
	FalloffAll = FalloffTopLeft | FalloffTopRight | FalloffLeft | FalloffRight,
	ScaleCenter = 0x200
};

UENUM()
enum class EToolTransformInterpType : uint8
{
	Linear,
	Sinusoidal,
	Cubic,
	CircularIn,
	CircularOut,
	ExpIn,
	ExpOut
};

ENUM_CLASS_FLAGS(ECurveEditorAnchorFlags);

struct FCurveEditorTransformWidget
{
	FCurveEditorTransformWidget()
	{
		SelectedAnchorFlags = ECurveEditorAnchorFlags::None;
		Visible = false;
	}

public:
	ECurveEditorAnchorFlags SelectedAnchorFlags;
	ECurveEditorAnchorFlags DisplayAnchorFlags;

	FGeometry MakeGeometry(const FGeometry& InWidgetGeometry) const
	{
		return InWidgetGeometry.MakeChild(Size, FSlateLayoutTransform(Position));
	}

	ECurveEditorAnchorFlags GetAnchorFlagsForMousePosition(const FGeometry& InWidgetGeometry, float FalloffHeight, float FalloffWidth, const FVector2D& RelativeScaleCenter, const FVector2D& InMouseScreenPosition) const;

	void GetCenterGeometry(const FGeometry& InWidgetGeometry, FGeometry& OutCenter) const;
	void GetSidebarGeometry(const FGeometry& InWidgetGeometry, FGeometry& OutLeft, FGeometry& OutRight, FGeometry& OutTop, FGeometry& OutBottom) const;
	void GetCornerGeometry(const FGeometry& InWidgetGeometry, FGeometry& OutTopLeft, FGeometry& OutTopRight, FGeometry& OutBottomLeft, FGeometry& OutBottomRight) const;
	void GetFalloffGeometry(const FGeometry& InWidgetGeometry, float FalloffHeight, float FalloffWidth, FGeometry& OutTopLeft, FGeometry& OutTopRight, FGeometry& OutLeft, FGeometry& OutRight) const;
	void GetScaleCenterGeometry(const FGeometry& InWidgetGeometry, FVector2D ScaleCenter, FGeometry& OutInputBoxGeometry) const;
	void GetCenterIndicatorGeometry(const FGeometry& InWidgetGeometry, FGeometry& OutCenterIndicatorGeometry) const;

	FVector2D Size;
	FVector2D Position;
	bool Visible;

	// stores the physical bounds instead of the marquee bounds, for scale calculations
	FVector2D BoundsSize;
	FVector2D BoundsPosition;

	FVector2D StartSize;
	FVector2D StartPosition;
};

USTRUCT()
struct FTransformToolOptions
{
	GENERATED_BODY()

	UPROPERTY(Transient, EditAnywhere, Category = ToolOptions)
	float UpperBound = 0.f;

	UPROPERTY(Transient, EditAnywhere, Category = ToolOptions)
	float LowerBound = 0.f;

	UPROPERTY(Transient, EditAnywhere, Category = ToolOptions)
	FFrameNumber LeftBound;

	UPROPERTY(Transient, EditAnywhere, Category = ToolOptions)
	FFrameNumber RightBound;

	UPROPERTY(Transient, EditAnywhere, Category = ToolOptions)
	FFrameNumber ScaleCenterX;

	UPROPERTY(Transient, EditAnywhere, Category = ToolOptions)
	float ScaleCenterY = 0.f;

	/** specifies the falloff type applied to curve selection */
	UPROPERTY(Transient, EditAnywhere, Category = ToolOptions, Meta = (ToolTip = "Interpolation type for soft selection (activate by holding ctrl)"))
	EToolTransformInterpType FalloffInterpType = EToolTransformInterpType::Linear;
};

class FCurveEditorTransformTool : public ICurveEditorToolExtension
{
public:
	FCurveEditorTransformTool(TWeakPtr<FCurveEditor> InCurveEditor)
		: WeakCurveEditor(InCurveEditor)
		, FalloffHeight(0.0f)
		, FalloffWidth(0.0f) 
		, RelativeScaleCenter(FVector2D(0.5f, 0.5f))
		, DisplayRelativeScaleCenter(FVector2D(0.5f, 0.5f))
	{
		ToolOptions.FalloffInterpType = EToolTransformInterpType::Linear;
	}

	// ICurveEditorToolExtension Interface
	virtual void OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 PaintOnLayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual void OnToolActivated() override;
	virtual void OnToolDeactivated() override;
	virtual void BindCommands(TSharedRef<FUICommandList> CommandBindings) override;
	virtual FReply OnMouseButtonDown(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnFocusLost(const FFocusEvent& InFocusEvent);
	TSharedPtr<FStructOnScope> GetToolOptions() const override { return ToolOptionsOnScope; }
	virtual void OnToolOptionsUpdated(const FPropertyChangedEvent& InPropertyChangedEvent) override;
	virtual FText GetLabel() const override;
	virtual FText GetDescription() const override;
	virtual FSlateIcon GetIcon() const override;
	// ~ICurveEditorToolExtension

private:
	void UpdateMarqueeBoundingBox();
	void UpdateToolOptions();
	void DrawMarqueeWidget(const FCurveEditorTransformWidget& InTransformWidget, const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InMyCullingRect, FSlateWindowElementList& OutDrawElements, const int32 InPaintOnLayerId, const FWidgetStyle& InWidgetStyle, const bool bInParentEnabled) const;
	void ScaleFrom(const FVector2D& InPanelSpaceCenter, const FVector2D& InChangeAmount, const bool bInFalloffOn, const bool bInAffectsX, const bool bInAffectsY);

	void OnDragStart();
	void OnDrag(const FPointerEvent& InMouseEvent, const FVector2D& InLocalMousePosition);
	void OnDragEnd();
	void StopDragIfPossible();

private:
	/** Weak pointer back to the Curve Editor this belongs to. */
	TWeakPtr<FCurveEditor> WeakCurveEditor;

	/** The currently open transaction (if any) */
	TUniquePtr<UE::CurveEditor::FScopedCurveChange> ActiveTransaction;

	/** Cached information about our transform tool such as interaction state, etc. */
	FCurveEditorTransformWidget TransformWidget;

	/** Set when attempting to move a drag handle. This allows us to tell the difference between a click and a click-drag. */
	TOptional<FDelayedDrag> DelayedDrag;
	/** Squishes the tangents vertically depending on how much we move the keys vertically. */
	TOptional<UE::CurveEditor::FTangentSelectionFlattener> TangentFlattener;

	/** Used to cache selected key data when doing transform operations */
	struct FKeyData
	{
		FKeyData(FCurveModelID InCurveID)
			: CurveID(InCurveID)
		{}

		/** The curve that contains the keys we're dragging */
		FCurveModelID CurveID;
		/** All the handles within a given curve that we are dragging */
		TArray<FKeyHandle> Handles;
		/** The extended key info for each of the above handles */
		TArray<FKeyPosition> StartKeyPositions;
		/** Used in OnEndDrag to send final key updates */
		TArray<FKeyPosition> LastDraggedKeyPositions;
	};

	/** Key dragging data stored per-curve */
	TArray<FKeyData> KeysByCurve;
	TRange<double> InputMinMax;

	FVector2D InitialMousePosition;
	FVector2D LastMousePosition;
	FCurveEditorAxisSnap::FSnapState SnappingState;

	/** UStruct that displays properties to be modified on screen */
	FTransformToolOptions ToolOptions;
	TSharedPtr<FStructOnScope> ToolOptionsOnScope;

private:
	/** Soft Selection Height, if 0.0 as we go up to 1.0 we go up to the top edge */
	float FalloffHeight;
	/** Soft Selection Width, if 0.0 we peak directly at the midpoint gradually scaling out till we reach 1.0fff */
	float FalloffWidth;

	/** Whether or not the selections have the same view transforms */
	bool bCurvesHaveSameScales;

	/** point to scale from, relative to current bounds */
	FVector2D RelativeScaleCenter;

	/** Same as the above, but flipped when the scale amount is inverted for display */
	FVector2D DisplayRelativeScaleCenter;

	/** Get Soft Select Change Based upon Input Value and internal soft selection weight values*/
	double GetFalloffWeight(double InputValue) const;

	/** Modify the Falloff Weight by type of Interpolation */
	double ModifyWeightByInterpType(double Weight) const;

	/** Selection serial number, used to update the toolbox */
	uint32 SelectionSerialNumber = 0;
};

