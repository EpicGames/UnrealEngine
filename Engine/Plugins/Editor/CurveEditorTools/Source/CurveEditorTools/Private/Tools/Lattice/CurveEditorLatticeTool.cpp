// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorLatticeTool.h"

#include "CoreGlobals.h"
#include "CurveEditor.h"
#include "CurveEditorToolCommands.h"
#include "Drag/LatticeDragOp_MoveControlPoints.h"
#include "Editor.h"
#include "LatticePointMetaData.h"
#include "Editor/TransBuffer.h"
#include "Internationalization/Text.h"
#include "Mirror/LatticeEdgeTangentMirrorOp.h"
#include "Mirror/LatticePointTangentMirrorOp.h"
#include "Misc/CurvePointSnapper.h"
#include "Misc/DeformerUtils.h"
#include "Misc/LatticeDrawUtils.h"
#include "Misc/LatticeUtils.h"
#include "Misc/Mirror/MirrorUtils.h"
#include "Misc/VectorMathUtils.h"
#include "SCurveEditorPanel.h"
#include "SCurveEditorView.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "Undo/LatticeUndoObject.h"

#define LOCTEXT_NAMESPACE "FCurveEditorLatticeTool"

namespace UE::CurveEditorTools
{
FCurveEditorLatticeTool::FCurveEditorLatticeTool(TWeakPtr<FCurveEditor> InCurveEditor)
	: WeakCurveEditor(InCurveEditor)
	, FinalMouseScreenSpacePosition()
	, UndoObject(
		NewObject<UCurveEditorTools_LatticeUndoObject>(GetTransientPackage(), UCurveEditorTools_LatticeUndoObject::StaticClass(), NAME_None, RF_Transactional)
	)
{
	UndoObject->OnPostEditUndo.AddLambda([this, CurveEditor = InCurveEditor]
	{
		const TSharedPtr<FCurveEditor> Editor = CurveEditor.Pin();
		if (!Editor)
		{
			return;
		}

		// LatticeDeformerState is unset if the user redo-es the first transaction which set the initial selection...
		if (LatticeDeformerState)
		{
			LatticeDeformerState->bHasRequestedRefresh = true;
			LatticeDeformerState->RefreshFlags |= ELatticeUpdateFlags::TryRestoreBoundsFromUndo;
			// If user undo-es the first transaction, next tick we'll notice that nothing is selected and destroy the deformer.
		}
	});
}

FCurveEditorLatticeTool::~FCurveEditorLatticeTool()
{
	UnregisterDelegates();
}

void FCurveEditorLatticeTool::Tick(const FGeometry& InViewContainerGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// DrawData needs to be updated in Tick. InViewContainerGeometry is slightly smaller in OnPaint than in OnTick/OnMouseButtonDown, etc.
	// This results in the hover state not being displayed correctly. For that reason, we generate the data in OnTick and draw it in OnPaint.
	const TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return;
	}
	
	if (LatticeDeformerState && LatticeDeformerState->bHasRequestedRefresh)
	{
		const ELatticeUpdateFlags Flags = LatticeDeformerState->RefreshFlags;
		LatticeDeformerState->RefreshFlags = ELatticeUpdateFlags::None;
		LatticeDeformerState->bHasRequestedRefresh = false;
		RecomputeLatticeGrid(CurveEditor.ToSharedRef(), Flags);
	}
	else
	{
		UpdateDeformerBounds(CurveEditor.ToSharedRef());
	}
		
	if (LatticeDeformerState.IsSet())
	{
		DrawData.Emplace();
		RecomputeDrawData(InViewContainerGeometry, CurveEditor);
	}
	else
	{
		DrawData.Reset();
	}
}

void FCurveEditorLatticeTool::OnPaint(
	const FPaintArgs& InPaintArgs,
	const FGeometry& InViewContainerGeometry,
	const FSlateRect& InMyCullingRect,
	FSlateWindowElementList& OutDrawElements,
	int32 InPaintOnLayerId,
	const FWidgetStyle& InWidgetStyle,
	bool bParentEnabled
	) const
{
	if (DrawData.IsSet())
	{
		DrawLatticeControls(*DrawData, InViewContainerGeometry, OutDrawElements, InPaintOnLayerId);
	}
}

FReply FCurveEditorLatticeTool::OnMouseButtonDoubleClick(
	TSharedRef<SWidget> OwningWidget, const FGeometry& InViewContainerGeometry, const FPointerEvent& InMouseEvent
	)
{
	using namespace UE::CurveEditor;
	CancelDrag();
	
	const bool bHandled = ComputeHoverStateForLeftClick(
		InViewContainerGeometry, InMouseEvent,
		[this](const FLatticeHoverState& HoverState, const FLatticeBounds&, const FCurveEditor& InCurveEditor)
		{
			// Double clicking the center will mirror everything with the midpoint being a horizontal line through center of the grid.
			if (HoverState.HoveredCell)
			{
				const FScopedTransaction Transaction(LOCTEXT("FlipCurves", "Flip curves"));
				const FScopedCurveChange KeyChange(
					FCurvesSnapshotBuilder(WeakCurveEditor, ECurveChangeFlags::KeyAttributes | ECurveChangeFlags::MoveKeys).TrackSelectedCurves()
					);
				UndoObject->Modify();
				
				HandleMirrorGrid();
				SaveUndoState(InCurveEditor);
				return true;
			}
			if (HoverState.HoveredEdge)
			{
				const FScopedTransaction Transaction(LOCTEXT("FlattenCurves", "Flatten curves"));
				const FScopedCurveChange KeyChange(
					FCurvesSnapshotBuilder(WeakCurveEditor, ECurveChangeFlags::KeyAttributes | ECurveChangeFlags::MoveKeys).TrackSelectedCurves()
					);
				UndoObject->Modify();
				
				HandleMoveEdgeToOpposite(*HoverState.HoveredEdge);
				SaveUndoState(InCurveEditor);
				return true;
			}
			return false;
		});
	
	return bHandled
		// Capturing the mouse is important so OnMouseButtonDown and OnMouseButtonUp are called on us instead of bubbling them up the hierarchy.
		// Without this, SInteractiveCurveEditorView::OnMouseButtonUp can cause a curve to be clicked and select all keys under it.
		? FReply::Handled().CaptureMouse(OwningWidget)
		: FReply::Unhandled();
}
	
FReply FCurveEditorLatticeTool::OnMouseButtonDown(
	TSharedRef<SWidget> InOwningWidget, const FGeometry& InViewContainerGeometry, const FPointerEvent& InMouseEvent
	)
{
	CancelDrag();
	const bool bHandled = ComputeHoverStateForLeftClick(InViewContainerGeometry, InMouseEvent,
		[this, &InMouseEvent](const FLatticeHoverState& HoverState, const FLatticeBounds& Bounds, const FCurveEditor&)
		{
			if (HoverState.IsHovered())
			{
				DelayedDragData.Emplace(
					InMouseEvent.GetScreenSpacePosition(),
					InMouseEvent.GetEffectingButton(),
					Bounds,
					HoverState
					);
				return true;
			}
			return false;
		});
	
	return bHandled ? FReply::Handled().CaptureMouse(InOwningWidget).PreventThrottling() : FReply::Unhandled();
}

	
FReply FCurveEditorLatticeTool::OnMouseButtonUp(
	TSharedRef<SWidget> InOwningWidget, const FGeometry& InViewContainerGeometry, const FPointerEvent& MouseEvent
	)
{
	if (DelayedDragData.IsSet())
	{
		FReply Reply = FReply::Handled();
		if (DelayedDragData->DelayedDrag.IsDragging() && ensure(DelayedDragData->ActiveOperation))
		{
			DelayedDragData->ActiveOperation->DragOp->EndDrag(InViewContainerGeometry, MouseEvent);
			// Only return handled if we actually started a drag
			Reply.ReleaseMouseCapture();
		}
		
		// Resetting DelayedDragData may cause FScopedCurveChange to invoke the curves' modified events... which would cause us to recompute our lattice grid.
		// It's important to do this after EndDrag is (potentially) called, as that sets LatticeDeformerState->bIsModifyingCurves to false.
		const TGuardValue Guard(LatticeDeformerState->bIsModifyingCurves, true);
		DelayedDragData.Reset();
		return Reply;
	}
	
	return FReply::Unhandled();
}

FReply FCurveEditorLatticeTool::OnMouseMove(
	TSharedRef<SWidget> InOwningWidget, const FGeometry& InViewContainerGeometry, const FPointerEvent& InMouseEvent
	)
{
	FinalMouseScreenSpacePosition = InMouseEvent.GetScreenSpacePosition();
	
	if (DelayedDragData.IsSet())
	{
		FReply Reply = FReply::Handled();

		if (DelayedDragData->DelayedDrag.IsDragging() && ensure(DelayedDragData->ActiveOperation))
		{
			DelayedDragData->ActiveOperation->DragOp->MoveMouse(InViewContainerGeometry, InMouseEvent);
		}
		else if (DelayedDragData->DelayedDrag.AttemptDragStart(InMouseEvent))
		{
			StartDragDrop(InViewContainerGeometry, InMouseEvent);
		}
			
		// Steal the capture, as we're now the authoritative widget in charge of a mouse-drag operation
		Reply.CaptureMouse(InOwningWidget);	
		return Reply;
	}
	
	return ICurveEditorToolExtension::OnMouseMove(InOwningWidget, InViewContainerGeometry, InMouseEvent);
}

void FCurveEditorLatticeTool::OnFinishedPointerInput()
{
	if (DelayedDragData && DelayedDragData->ActiveOperation)
	{
		DelayedDragData->ActiveOperation->DragOp->FinishedPointerInput();
	}
}

void FCurveEditorLatticeTool::OnToolActivated()
{
	if (UTransBuffer* Transactor = GEditor ? Cast<UTransBuffer>(GEditor->Trans) : nullptr)
	{
		Transactor->OnTransactionStateChanged().AddRaw(this, &FCurveEditorLatticeTool::OnTransactionStateChanged);
	}

	if (const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin())
	{
		CurveEditorPin->Selection.OnSelectionChanged().AddRaw(this, &FCurveEditorLatticeTool::OnSelectionChanged);
	}
}

void FCurveEditorLatticeTool::OnToolDeactivated()
{
	// Clear data. We'll recompute a new grid next time the user activate the tool
	LatticeDeformerState.Reset();
	UnregisterDelegates();
}

void FCurveEditorLatticeTool::BindCommands(TSharedRef<FUICommandList> CommandBindings)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor)
	{
		FIsActionChecked MultiScaleToolIsActive = FIsActionChecked::CreateSP(CurveEditor.ToSharedRef(), &FCurveEditor::IsToolActive, ToolID);
		FExecuteAction ActivateMultiScaleTool = FExecuteAction::CreateSP(CurveEditor.ToSharedRef(), &FCurveEditor::MakeToolActive, ToolID);

		CommandBindings->MapAction(
			FCurveEditorToolCommands::Get().ActivateLatticeTool, ActivateMultiScaleTool, FCanExecuteAction(), MultiScaleToolIsActive
			);
	}
}

FText FCurveEditorLatticeTool::GetLabel() const
{
	return FCurveEditorToolCommands::Get().ActivateLatticeTool->GetLabel();
}

FText FCurveEditorLatticeTool::GetDescription() const
{
	return FCurveEditorToolCommands::Get().ActivateLatticeTool->GetDescription();
}

FSlateIcon FCurveEditorLatticeTool::GetIcon() const
{
	return FCurveEditorToolCommands::Get().ActivateLatticeTool->GetIcon();
}

void FCurveEditorLatticeTool::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(UndoObject);
}

void FCurveEditorLatticeTool::UpdateDeformerBounds(const TSharedRef<FCurveEditor>& InCurveEditor)
{
	if (LatticeDeformerState.IsSet())
	{
		return;
	}

	// Handle the edge case of all points overlapping, or a single key selection.
	// The lattice math does not work when widht or height are 0 due to division by 0; fyi, the deformer handles it with an ensure.
	const FLatticeBounds Bounds = ComputeBounds(*InCurveEditor);
	if (Bounds.bIsVisible && !IsLatticeTooSmall(Bounds.MinValues, Bounds.MaxValues) && InCurveEditor->Selection.Count() > 1)
	{
		LatticeDeformerState.Emplace(Bounds, InCurveEditor);
		LatticeDeformerState->CurveChangeListener.OnCurveModified().AddRaw(this, &FCurveEditorLatticeTool::OnCurvesModified);
		LatticeDeformerState->PanelRebuildListener.OnCurveViewRebuilt().AddRaw(this, &FCurveEditorLatticeTool::OnCurvePanelRebuilt);
		SaveUndoState(*InCurveEditor);
	}
}

void FCurveEditorLatticeTool::OnCurvesModified()
{
	// The modification was driven by us?
	if (!LatticeDeformerState->bIsModifyingCurves)
	{
		LatticeDeformerState->bHasRequestedRefresh = true;
	}
}

void FCurveEditorLatticeTool::OnSelectionChanged()
{
	const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin();
	if (!CurveEditorPin)
	{
		return;
	}

	// A change in selection can be caused by the lattice tool: if the user stacked some keys, the stacked keys are removed. That causes a selection change.
	// In that case, retain the shape of the lattice. Just save the current shape into the undo buffer so it can be restored.
	if (LatticeDeformerState && LatticeDeformerState->bIsModifyingCurves)
	{
		SaveUndoState(*CurveEditorPin);
	}
	else if (GIsTransacting && LatticeDeformerState)
	{
		LatticeDeformerState->bHasRequestedRefresh = true;
	}
	else
	{
		CancelDrag();
		RecomputeLatticeGrid(CurveEditorPin.ToSharedRef());
	}
}

void FCurveEditorLatticeTool::OnTransactionStateChanged(
	const FTransactionContext& Context, ETransactionStateEventType TransactionStateEvent
	)
{
	UTransBuffer* TransBuffer = GEditor ? CastChecked<UTransBuffer>(GEditor->Trans) : nullptr;
	if (!TransBuffer || TransactionStateEvent != ETransactionStateEventType::UndoRedoFinalized)
	{
		return;
	}
	const TSharedRef<FTransaction>* CurrentTransaction = TransBuffer->UndoBuffer.FindByPredicate(
		[&Context](const TSharedRef<FTransaction>& Transaction)
		{
			return Transaction->GetOperationId() == Context.OperationId;
		});
	
	// We're trying to handle cases where curves are changed but OnCurvesModified, i.e. FCurveModel::OnCurveModified, is not triggered...
	// This happens e.g. when underlying Sequencer data changes, which is under the hood of the FCurveModel interface.
	// We'll just always regenerat the grid, i.e. a new rectangular lattice will be placed around the selection.
	//
	// Sadly, this means that undos / redos on totally unrelated places in the engine (e.g. undo an edit operation in details panel), will cause the
	// lattice to be regenerated, too. This is fine as long as it does not happen while the user is actively working in the curve editor.
	// This is why we check whether the UndoObject is in the current transaction... in that case we do NOT recompute the lattice since our custom
	// undo logic may change control points directly.
	const bool bAppliesToLattice = CurrentTransaction && CurrentTransaction->Get().ContainsObject(UndoObject);
	if (!bAppliesToLattice)
	{
		RecomputeLatticeGrid();
	}
}

void FCurveEditorLatticeTool::RecomputeLatticeGrid(const TSharedRef<FCurveEditor>& InCurveEditor, ELatticeUpdateFlags InRefreshFlags)
{
	if (EnumHasAnyFlags(InRefreshFlags, ELatticeUpdateFlags::TryRestoreBoundsFromUndo)
		&& ApplyUndoState(*InCurveEditor))
	{
		return;
	}
	
	// This will calculate a new rectangular grid around the points.
	// In the past, we tried to preserve the existing shape by just moving it over the moved keys but there were too many different tools interacting
	// which made this difficult to get right in every case. Hence, we decided just to recompute the grid on move.
	LatticeDeformerState.Reset();
	UpdateDeformerBounds(InCurveEditor);
}
	
void FCurveEditorLatticeTool::RecomputeDrawData(const FGeometry& InViewContainerGeometry, const TSharedPtr<FCurveEditor>& InCurveEditor)
{
	// The widget hierarchy is SCurveEditorViewContainer -> SCurveEditorViewAbsolute.
	// InAllottedGeometry is that of SCurveEditorViewContainer.
	DrawData->MatrixWidth = GetGlobalDeformer().NumPointsInWidth();
	DrawData->ControlPoints = ComputeDrawControlPoints(*InCurveEditor);
	const bool bHasControlData = !DrawData->ControlPoints.IsEmpty();
	if (!bHasControlData)
	{
		return;
	}
	
	DrawData->ControlEdges = ComputeDrawControlEdges(DrawData->ControlPoints);
	// No hover highlights while dragging (or while the user is doing anything with mouse buttons down in the application).
	const bool bAnyMouseButtonsDown = !FSlateApplication::Get().GetPressedMouseButtons().IsEmpty();
	if (!DelayedDragData.IsSet() && !bAnyMouseButtonsDown)
	{
		DrawData->HoverState = ComputeLatticeHoverState(
			InViewContainerGeometry, FinalMouseScreenSpacePosition,
			DrawData->ControlPoints,DrawData->ControlEdges,
			GetGlobalDeformer().NumPointsInWidth(), GetGlobalDeformer().NumCells()
		);
	}
}

TArray<FVector2D> FCurveEditorLatticeTool::ComputeDrawControlPoints(const FCurveEditor& InCurveEditor) const
{
	check(LatticeDeformerState);
	TArray<FVector2D> Result;
	
	const TArray<FVector2D> ControlPoints = LatticeDeformerState->TransformControlPointsToCurveSpace();
	Result.SetNumUninitialized(ControlPoints.Num());
	if (!TransformViewToViewContainer(InCurveEditor, ControlPoints, Result))
	{
		return {};
	}
	return Result;
}

TArray<FLatticeControlEdge> FCurveEditorLatticeTool::ComputeDrawControlEdges(TConstArrayView<FVector2D> InControlPoints) const
{
	TArray<FLatticeControlEdge> ControlEdges;
	ControlEdges.SetNumUninitialized(GetGlobalDeformer().NumEdges());
	for (int32 EdgeIndex = 0; EdgeIndex < GetGlobalDeformer().NumEdges(); ++EdgeIndex)
	{
		const auto[Start, End] = GetGlobalDeformer().GetEdgeIndices(EdgeIndex);
		ControlEdges[EdgeIndex] = FLatticeControlEdge(InControlPoints[Start], InControlPoints[End]);
	}
	return ControlEdges;
}

static FText LexViewMode(ECurveEditorViewID ViewMode)
{
	switch (ViewMode)
	{
	case ECurveEditorViewID::Absolute: return LOCTEXT("Absolute", "Absolute");
	case ECurveEditorViewID::Normalized: return LOCTEXT("Normalized", "Normalized");
	case ECurveEditorViewID::Stacked: return LOCTEXT("Stacked", "Stacked");
	default: return LOCTEXT("Custom", "Custom");
	}
}
	
void FCurveEditorLatticeTool::CancelDrag()
{
	if (DelayedDragData && DelayedDragData->ActiveOperation)
	{
		DelayedDragData->ActiveOperation->DragOp->CancelDrag();
	}
	DelayedDragData.Reset();
}
	
void FCurveEditorLatticeTool::StartDragDrop(const FGeometry& InViewContainerGeometry, const FPointerEvent& InMouseEvent)
{
	if (TOptional<FDragOpData> DragDrop = CreateDragDrop(DelayedDragData->Bounds, DelayedDragData->HoverState))
	{
		DelayedDragData->ActiveOperation = MoveTemp(*DragDrop);
		DelayedDragData->ActiveOperation->DragOp->BeginDrag(
			InViewContainerGeometry, InMouseEvent, DelayedDragData->DelayedDrag.GetInitialPosition()
		);
	}
}
	
TOptional<FDragOpData> FCurveEditorLatticeTool::CreateDragDrop(
	const FLatticeBounds& InBounds, const FLatticeHoverState& InHoverState
	)
{
	if (InHoverState.HoveredControlPoint)
	{
		return CreateDragDrop_Point(InBounds, *InHoverState.HoveredControlPoint);
	}
	if (InHoverState.HoveredEdge)
	{
		return CreateDragDrop_Edge(InBounds, *InHoverState.HoveredEdge);
	}
	if (InHoverState.HoveredCell)
	{
		return CreateDragDrop_Cell(InBounds, *InHoverState.HoveredCell);
	}

	return {};
}

TOptional<FDragOpData> FCurveEditorLatticeTool::CreateDragDrop_Point(
	const FLatticeBounds& InBounds, const int32 InControlPointIndex
	)
{
	using FHandleMovedPoints = FLatticeDragOp_MoveControlPoints::FOnControlPointsMoved;
	const TConstArrayView<FVector2D> ControlPoints = GetGlobalDeformer().GetControlPoints();
	
	const auto MakeDelegate = [this, InControlPointIndex](EPropertyChangeType::Type ChangeType)
	{
		return FHandleMovedPoints::CreateLambda(
			[this, InControlPointIndex, ChangeType](const TConstArrayView<FVector2D>& InNewControlPointPositions, EKeyMoveMode InSnapMode)
			{
				const int32 Indices[1] { InControlPointIndex };
				HandleDragPoints(&FCurveEditorLatticeTool::OnMoveControlPoint,Indices, InNewControlPointPositions, ChangeType, InSnapMode);
				GetGlobalDeformer().UpdateControlPoints(Indices, InNewControlPointPositions, [](auto, auto){});

				// When the drag is finished, save where the lattice is so its shape can be transacted correctly.
				const bool bIsDragFinished = ChangeType == EPropertyChangeType::ValueSet; 
				if (bIsDragFinished)
				{
					LatticeDeformerState->bIsModifyingCurves = false;
					SaveUndoState(*WeakCurveEditor.Pin());
				}
			});
	};
	const FSimpleDelegate HandleBeginDrag = FSimpleDelegate::CreateLambda([this](){ LatticeDeformerState->bIsModifyingCurves = true; });
	const FHandleMovedPoints HandleDrag = MakeDelegate(EPropertyChangeType::Interactive);
	const FHandleMovedPoints HandleFinishDrag = MakeDelegate(EPropertyChangeType::ValueSet);
	
	TUniquePtr<FLatticeDragOp> Op = MakeUnique<FLatticeDragOp_MoveControlPoints>(
		WeakCurveEditor, FControlPointArray{ ControlPoints[InControlPointIndex] }, InBounds, HandleBeginDrag, HandleDrag, HandleFinishDrag
	);
	FDragOpData DragDrop(
		WeakCurveEditor, MoveTemp(Op),
		FText::Format(LOCTEXT("MoveControlPoint", "Move control point ({0})"), LexViewMode(WeakCurveEditor.Pin()->GetPanel()->GetViewMode()))
		);

	const FCurveEditor& CurveEditor = *WeakCurveEditor.Pin();
	for (const TPair<FCurveModelID, FPerCurveLatticeData>& Pair : LatticeDeformerState->PerCurveData)
	{
		const FPerCurveDeformer2D& Deformer = Pair.Value.CurveDeformer;
		TOptional<FLatticePointTangentsMirrorOp> MirrorOp = FLatticePointTangentsMirrorOp::MakeMirrorOpForDragLatticeControlPoint(
			InControlPointIndex, Pair.Key, Deformer, CurveEditor
		);
		if (MirrorOp)
		{
			DragDrop.PointTangentMirroringOps.Add(Pair.Key, MoveTemp(*MirrorOp));
		}
	}
	return DragDrop;
}

TOptional<FDragOpData> FCurveEditorLatticeTool::CreateDragDrop_Edge(
	const FLatticeBounds& InBounds, int32 InEdgeIndex
	)
{
	using FHandleMovedPoints = FLatticeDragOp_MoveControlPoints::FOnControlPointsMoved;
	const TConstArrayView<FVector2D> ControlPoints = GetGlobalDeformer().GetControlPoints();
	const FEdgeVertexIndices EdgeIndices = GetGlobalDeformer().GetEdgeIndices(InEdgeIndex);
	const auto MakeDelegate = [this, EdgeIndices](EPropertyChangeType::Type ChangeType)
	{
		return FHandleMovedPoints::CreateLambda(
			[this, EdgeIndices, ChangeType](const TConstArrayView<FVector2D>& InNewControlPointPositions, EKeyMoveMode InSnapMode)
			{
				const int32 Indices[2]{ EdgeIndices.Start, EdgeIndices.End };
				HandleDragPoints(&FCurveEditorLatticeTool::OnMoveEdge, Indices, InNewControlPointPositions, ChangeType, InSnapMode);
				GetGlobalDeformer().UpdateControlPoints(Indices, InNewControlPointPositions, [](auto, auto){});

				// When the drag is finished, save where the lattice is so its shape can be transacted correctly.
				const bool bIsDragFinished = ChangeType == EPropertyChangeType::ValueSet; 
				if (bIsDragFinished)
				{
					LatticeDeformerState->bIsModifyingCurves = false;
					SaveUndoState(*WeakCurveEditor.Pin());
				}
			});
	};
	const FSimpleDelegate HandleBeginDrag = FSimpleDelegate::CreateLambda([this](){ LatticeDeformerState->bIsModifyingCurves = true; });
	const FHandleMovedPoints HandleDrag = MakeDelegate(EPropertyChangeType::Interactive);
	const FHandleMovedPoints HandleFinishDrag = MakeDelegate(EPropertyChangeType::ValueSet);
		
	FControlPointArray EdgePoints{ ControlPoints[EdgeIndices.Start], ControlPoints[EdgeIndices.End] };
	TUniquePtr<FLatticeDragOp> Op = MakeUnique<FLatticeDragOp_MoveControlPoints>(
		WeakCurveEditor, MoveTemp(EdgePoints), InBounds, HandleBeginDrag, HandleDrag, HandleFinishDrag
	);
	FDragOpData DragDrop(
		WeakCurveEditor, MoveTemp(Op),
		FText::Format(LOCTEXT("MoveEdge", "Move edge ({0})"), LexViewMode(WeakCurveEditor.Pin()->GetPanel()->GetViewMode()))
		);

	const FCurveEditor& CurveEditor = *WeakCurveEditor.Pin();
	for (const TPair<FCurveModelID, FPerCurveLatticeData>& Pair : LatticeDeformerState->PerCurveData)
	{
		const FPerCurveDeformer2D& Deformer = Pair.Value.CurveDeformer;
		TOptional<FLatticeEdgeTangentsMirrorOp> MirrorOp = FLatticeEdgeTangentsMirrorOp::MakeMirrorOpForDragLatticeEdge(
			InEdgeIndex, Pair.Key, Deformer, CurveEditor
			);
		if (MirrorOp)
		{
			DragDrop.EdgeTangentMirroringOps.Emplace(Pair.Key, MoveTemp(*MirrorOp));
		}
	}
	return DragDrop;
}

TOptional<FDragOpData> FCurveEditorLatticeTool::CreateDragDrop_Cell(
	const FLatticeBounds& InBounds, int32 InCellIndex
	)
{
	using FHandleMovedPoints = FLatticeDragOp_MoveControlPoints::FOnControlPointsMoved;
	const TConstArrayView<FVector2D> ControlPoints = GetGlobalDeformer().GetControlPoints();
	
	const FCellVertexIndices CellIndices = GetGlobalDeformer().GetCellIndices(InCellIndex);
	const auto MakeDelegate = [this, CellIndices](EPropertyChangeType::Type ChangeType)
	{
		return FHandleMovedPoints::CreateLambda(
			[this, CellIndices, ChangeType](const TConstArrayView<FVector2D>& InNewControlPointPositions, EKeyMoveMode InSnapMode)
			{
				const int32 Indices[4]{ CellIndices.TopLeft, CellIndices.TopRight, CellIndices.BottomRight, CellIndices.BottomLeft };
				HandleDragPoints(&FCurveEditorLatticeTool::OnSetControlPoints, Indices, InNewControlPointPositions, ChangeType, InSnapMode);
				GetGlobalDeformer().UpdateControlPoints(Indices, InNewControlPointPositions, [](auto, auto){});
				
				// When the drag is finished, save where the lattice is so its shape can be transacted correctly.
				const bool bIsDragFinished = ChangeType == EPropertyChangeType::ValueSet; 
				if (bIsDragFinished)
				{
					LatticeDeformerState->bIsModifyingCurves = false;
					SaveUndoState(*WeakCurveEditor.Pin());
				}
			});
	};
	const FSimpleDelegate HandleBeginDrag = FSimpleDelegate::CreateLambda([this](){ LatticeDeformerState->bIsModifyingCurves = true; });
	const FHandleMovedPoints HandleDrag = MakeDelegate(EPropertyChangeType::Interactive);
	const FHandleMovedPoints HandleFinishDrag = MakeDelegate(EPropertyChangeType::ValueSet);
		
	const auto[TopLeft, TopRight, BottomRight, BottomLeft] = CellIndices;
	FControlPointArray CellPoints{ ControlPoints[TopLeft], ControlPoints[TopRight], ControlPoints[BottomRight], ControlPoints[BottomLeft] };
	TUniquePtr<FLatticeDragOp> Op = MakeUnique<FLatticeDragOp_MoveControlPoints>(
		WeakCurveEditor, MoveTemp(CellPoints), InBounds, HandleBeginDrag, HandleDrag, HandleFinishDrag
	);
	return FDragOpData(
		WeakCurveEditor, MoveTemp(Op),
		FText::Format(LOCTEXT("MoveCell", "Move cell ({0})"), LexViewMode(WeakCurveEditor.Pin()->GetPanel()->GetViewMode()))
		);
}

void FCurveEditorLatticeTool::HandleDragPoints(
	FMovePointsSignature InMovePointsCallback,
	TConstArrayView<int32> InVertexIndices, TConstArrayView<FVector2D> InNewControlPoints,
	EPropertyChangeType::Type InChangeType, EKeyMoveMode InSnapMode
	)
{
	const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin();
	if (!CurveEditorPin)
	{
		return;
	}

	if (!DelayedDragData->ActiveOperation->bSavedUndoState)
	{
		DelayedDragData->ActiveOperation->bSavedUndoState = true;
		UndoObject->Modify();
	}
	
	for (TPair<FCurveModelID, FPerCurveLatticeData>& Pair : LatticeDeformerState->PerCurveData)
	{
		FCurveModel* CurveModel = CurveEditorPin->FindCurve(Pair.Key);
		if (!CurveModel)
		{
			continue;
		}

		// InNewControlPoints are in absolute space (literal FKeyPosition::InputValue / OutputValue).
		// Each curve's lattice is still in absolute space but depending on the view, the lattice may have been adjusted to a smaller area.
		// We need to transform the dragged control points to the space used by the per-curve lattice.
		const TArray<FVector2D> CurveRelativeControlPoints = TransformPoints(Pair.Value.GlobalDeformerToCurveDeformer, InNewControlPoints);
		
		FPerCurveDeformer2D& PerCurveDeformer = Pair.Value.CurveDeformer;
		(this->*InMovePointsCallback)(PerCurveDeformer, *CurveModel, InVertexIndices, CurveRelativeControlPoints, InChangeType, InSnapMode);
	}
}

void FCurveEditorLatticeTool::HandleMoveEdgeToOpposite(int32 InEdgeIndex)
{
	const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin();
	const TOptional<FMoveEdgeToOppositeData> MoveData = ComputeMoveEdgeToOppositeData(GetGlobalDeformer(), InEdgeIndex);
	if (!CurveEditorPin || !MoveData)
	{
		return;
	}
	
	// Move the per-curve deformers.
	for (TPair<FCurveModelID, FPerCurveLatticeData>& Pair : LatticeDeformerState->PerCurveData)
	{
		if (FCurveModel* Curve = CurveEditorPin->FindCurve(Pair.Key))
		{
			MoveEdgeToOpposite(Pair.Value.CurveDeformer, *Curve, *CurveEditorPin, InEdgeIndex);
		}
	}

	// Move the UI only deformer.
	const auto NoOp = [](auto, auto){};
	GetGlobalDeformer().UpdateControlPoints({ MoveData->EdgeIndices.Start, MoveData->EdgeIndices.End }, MoveData->EdgeControlPoints.Points, NoOp);
}

void FCurveEditorLatticeTool::HandleMirrorGrid()
{
	const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin();
	if (!CurveEditorPin)
	{
		return;
	}
	
	// 1. Mirror all the keys
	for (TPair<FCurveModelID, FPerCurveLatticeData>& Pair : LatticeDeformerState->PerCurveData)
	{
		if (FCurveModel* Curve = CurveEditorPin->FindCurve(Pair.Key))
		{
			MirrorGrid(Pair.Value.CurveDeformer, *Curve, *CurveEditorPin);
		}
	}
	
	// 2. Recompute lattice grid
	// This will calculate a new rectangular grid around the points. Instead, we could try to preserve the existing shape... but nobody cares, so lets not implement it.
	LatticeDeformerState.Reset();
	UpdateDeformerBounds(CurveEditorPin.ToSharedRef());
}

void FCurveEditorLatticeTool::OnMoveControlPoint(
	FPerCurveDeformer2D& InDeformer, FCurveModel& InCurveModel,
	TConstArrayView<int32> InVertexIndices, TConstArrayView<FVector2D> InNewControlPointPositions,
	EPropertyChangeType::Type InChangeType, EKeyMoveMode InSnapMode
	)
{
	check(InVertexIndices.Num() == 1 && InNewControlPointPositions.Num() == 1);
	ensure(LatticeDeformerState->bIsModifyingCurves);
	
	const FCurveModelID& CurveId = *InCurveModel.GetId();
	if (FLatticePointTangentsMirrorOp* MirrorOp = DelayedDragData->ActiveOperation->PointTangentMirroringOps.Find(CurveId))
	{
		MirrorOp->OnMovePoint(InNewControlPointPositions[0], *WeakCurveEditor.Pin());
	}
	
	OnSetControlPoints(InDeformer, InCurveModel, InVertexIndices, InNewControlPointPositions, InChangeType, InSnapMode);
}

void FCurveEditorLatticeTool::OnMoveEdge(
	FPerCurveDeformer2D& InDeformer, FCurveModel& InCurveModel,
	TConstArrayView<int32> InVertexIndices, TConstArrayView<FVector2D> InNewControlPointPositions,
	EPropertyChangeType::Type InChangeType, EKeyMoveMode InSnapMode
	)
{
	check(InVertexIndices.Num() == 2 && InNewControlPointPositions.Num() == 2);
	ensure(LatticeDeformerState->bIsModifyingCurves);

	const FCurveModelID& CurveId = *InCurveModel.GetId();
	if (FLatticeEdgeTangentsMirrorOp* MirrorOp = DelayedDragData->ActiveOperation->EdgeTangentMirroringOps.Find(CurveId))
	{
		MirrorOp->OnMoveEdge(InNewControlPointPositions, *WeakCurveEditor.Pin());
	}

	OnSetControlPoints(InDeformer, InCurveModel, InVertexIndices, InNewControlPointPositions, InChangeType, InSnapMode);
}

void FCurveEditorLatticeTool::OnSetControlPoints(
	FPerCurveDeformer2D& InDeformer, FCurveModel& InCurveModel,
	TConstArrayView<int32> InControlPointsToUpdate, TConstArrayView<FVector2D> InNewControlPoints,
	EPropertyChangeType::Type InChangeType, EKeyMoveMode InSnapMode
	)
{
	ensure(LatticeDeformerState->bIsModifyingCurves);
	
	const TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	const TOptional<FCurvePointSnapper> Snapper = CurveEditor ? FCurvePointSnapper::MakeSnapper(*CurveEditor, InCurveModel.GetOrInitId()) : TOptional<FCurvePointSnapper>{};
	if (!CurveEditor || !Snapper)
	{
		return;
	}
	
	// Combining Handles & Keys into large arrays like this is significantly faster than feeding each key into SetKeyPositions separately.
	// For 9000 keys it goes from 2136 ms to 238 ms.
	TArray<FKeyHandle> Handles;
	TArray<FKeyPosition> Keys;
	
	// Pre-allocating saves about 0.060ms (also it is needed because UpdateControlPoints is executed in parallel).
	const int32 NumKeys = InDeformer.NumKeys();
	Handles.Reserve(NumKeys);
	Keys.Reserve(NumKeys);
	
	InDeformer.UpdateControlPoints(InControlPointsToUpdate, InNewControlPoints,
		[InSnapMode, &Snapper, &Handles, &Keys](const FLatticePointMetaData& MetaData, const FVector2D& KeyPosition)
		{
			Handles.Add(MetaData.KeyHandle);
			const FKeyPosition& Original = MetaData.OriginalPosition;
			const FKeyPosition Position
			{
				EnumHasAnyFlags(InSnapMode, EKeyMoveMode::IgnoreX) ? Original.InputValue : KeyPosition.X,
				EnumHasAnyFlags(InSnapMode, EKeyMoveMode::IgnoreY) ? Original.OutputValue : KeyPosition.Y,
			};
			Keys.Add(Snapper->SnapKey(Position, InSnapMode));
		});

	// Some stats all with 9000 keys. Varying degree of changed keys, i.e. how far you move your mouse in quick succession.
	// Sorted by little to a lot of changes:
	// - SetKeyPositions: 4.299 ms		- Total: 4.533 ms
	// - SetKeyPositions 138.954 ms		- Total 139.229 ms
	// - SetKeyPositions 269.523 ms		- Total 269.811 ms
	// Conclusion: The following SetKeyPositions block is slow.
	InCurveModel.SetKeyPositions(Handles, Keys, InChangeType);
}

void FCurveEditorLatticeTool::MoveEdgeToOpposite(FPerCurveDeformer2D& InDeformer, FCurveModel& InCurveModel, FCurveEditor& InCurveEditor, int32 InEdgeIndex)
{
	const TOptional<FMoveEdgeToOppositeData> MoveData = ComputeMoveEdgeToOppositeData(InDeformer, InEdgeIndex);
	if (!MoveData)
	{
		return;
	}
	
	const TGuardValue Guard(LatticeDeformerState->bIsModifyingCurves, true);
	
	// Flatten tangents if moving top or bottom edge...
	const FCurveEditor& CurveEditor = *WeakCurveEditor.Pin();
	TOptional<FLatticeEdgeTangentsMirrorOp> MirrorOp = FLatticeEdgeTangentsMirrorOp::MakeMirrorOpForDragLatticeEdge(
		MoveData->OppositeEdgeIndex, *InCurveModel.GetId(), InDeformer, CurveEditor
		);
	if (MirrorOp) 
	{
		MirrorOp->OnMoveEdge(MoveData->EdgeControlPoints.Points, CurveEditor);
	}
	
	// ... then move the grid
	OnSetControlPoints(
		InDeformer, InCurveModel,
		{ MoveData->EdgeIndices.Start, MoveData->EdgeIndices.End }, MoveData->EdgeControlPoints.Points,
		EPropertyChangeType::ValueSet
		);
}

void FCurveEditorLatticeTool::MirrorGrid(FPerCurveDeformer2D& InDeformer, FCurveModel& InCurveModel, FCurveEditor& InCurveEditor)
{
	// At time of implementation, we only have a single cell. Should that change, adjust this implementation.
	if (InDeformer.NumCells() != 1)
	{
		return;
	}
	
	TArray<FKeyHandle> Keys;
	const TConstArrayView<FLatticePointMetaData> CellMetaData = InDeformer.GetCellMetaData(0);
	Algo::Transform(CellMetaData, Keys, [](const FLatticePointMetaData& MetaData){ return MetaData.KeyHandle; });
	
	const auto[Min, Max] = CurveEditor::FindMinMaxHeight(Keys, InCurveModel);
	const double Midpoint = Min + (Max - Min) * 0.5;
	CurveEditor::MirrorTangents(Keys, *InCurveModel.GetId(), Min, Max, Midpoint, InCurveEditor);
	CurveEditor::MirrorKeyPositions(Keys, *InCurveModel.GetId(), Min, Max, Midpoint, InCurveEditor);
}

template <typename TCallback> requires std::is_invocable_r_v<bool, TCallback, const FLatticeHoverState&, const FLatticeBounds&, const FCurveEditor&>
bool FCurveEditorLatticeTool::ComputeHoverStateForLeftClick(
	const FGeometry& InViewContainerGeometry, const FPointerEvent& InMouseEvent, TCallback&& InCallback
	)
{
	const TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor || !LatticeDeformerState.IsSet())
	{
		return false;
	}
	
	const FLatticeBounds Bounds = ComputeBounds(*CurveEditor);
	if (Bounds.bIsVisible && InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		const TArray<FVector2D> ControlPointsInSlate = ComputeDrawControlPoints(*CurveEditor);
		const FLatticeHoverState HoverState = ComputeLatticeHoverState(
			InViewContainerGeometry, InMouseEvent.GetScreenSpacePosition(),
			ControlPointsInSlate, ComputeDrawControlEdges(ControlPointsInSlate),
			GetGlobalDeformer().NumPointsInWidth(), GetGlobalDeformer().NumCells()
			);
		return InCallback(HoverState, Bounds, *CurveEditor);
	}

	return false;
}
	
void FCurveEditorLatticeTool::UnregisterDelegates() const
{
	if (UTransBuffer* Transactor = GEditor ? Cast<UTransBuffer>(GEditor->Trans) : nullptr)
	{
		Transactor->OnTransactionStateChanged().RemoveAll(this);
	}

	if (const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin())
	{
		CurveEditorPin->Selection.OnSelectionChanged().RemoveAll(this);
	}
}

void FCurveEditorLatticeTool::SaveUndoState(const FCurveEditor& InCurveEditor)
{
	if (ensure(LatticeDeformerState) && GUndo)
	{
		UndoObject->Modify();
		UndoObject->Snapshot = TakeSnapshot(
			InCurveEditor,
			LatticeDeformerState->ControlPointToCurveSpace, LatticeDeformerState->GlobalDeformer, LatticeDeformerState->PerCurveData
			);
	}
}

bool FCurveEditorLatticeTool::ApplyUndoState(FCurveEditor& InCurveEditor)
{
	if (!LatticeDeformerState)
	{
		return false;
	}
	
	const bool bWereViewModesCompatible = ApplySnapshot(
		UndoObject->Snapshot, InCurveEditor,
		LatticeDeformerState->ControlPointToCurveSpace,
		LatticeDeformerState->GlobalDeformer, LatticeDeformerState->PerCurveData
		);

	// The snapshot could e.g. have been taken in Absolute view mode, but we might currently be in Normalized view.
	// Since the lattice shapes are incompatible between modes, we'll just regenerate the entire grid.
	if (!bWereViewModesCompatible)
	{
		LatticeDeformerState.Reset();
	}

	return bWereViewModesCompatible;
}
}

#undef LOCTEXT_NAMESPACE