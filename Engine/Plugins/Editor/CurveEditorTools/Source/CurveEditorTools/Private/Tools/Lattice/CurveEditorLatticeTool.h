// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Drag/LatticeDragOp.h"
#include "Framework/DelayedDrag.h"
#include "ICurveEditorToolExtension.h"
#include "LatticeControlsDrawData.h"
#include "LatticeDeformer2D.h"
#include "LatticeDeformerState.h"
#include "LatticeFwd.h"
#include "Mirror/LatticeEdgeTangentMirrorOp.h"
#include "Mirror/LatticePointTangentMirrorOp.h"
#include "Misc/CurveChangeListener.h"
#include "Misc/CurveViewRebuildListener.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/LatticeDrawUtils.h"
#include "PerCurveLatticeData.h"
#include "ScopedTransaction.h"
#include "Drag/DelayedDragData.h"
#include "Modification/Utils/ScopedCurveChange.h"
#include "Templates/SharedPointer.h"
#include "UObject/UnrealType.h"

class FCurveEditor;
class FText;
class UCurveEditorTools_LatticeUndoObject;
struct FTransactionContext;
enum class ETransactionStateEventType : uint8;

namespace UE::CurveEditorTools
{
struct FLatticeBounds;

/**
 * Places a lattice deformer over the selection.
 * 
 * The lattice is a quadrilateral you can reshape by moving the points or edges. Reshaping the quad recomputes the underlying key positions.
 * The lattice does mirroring when:
 * - Dragging top edge down / bottom edge up: As the edge moves closer to the opposite edge, user set tangents are flattened and then inverted.
 * - Dragging a control point to the top / bottom edge: Similar as the previous case but a fallout is applied to the key to the left / right of the control point.
 *
 * See documentation on FLatticeDeformerState::Deformer and FLatticeDeformerState::PerCurveData to understand the linear spaces the lattice math operates on.
 */
class FCurveEditorLatticeTool : public ICurveEditorToolExtension, public FGCObject
{
public:
	
	explicit FCurveEditorLatticeTool(TWeakPtr<FCurveEditor> InCurveEditor);
	virtual ~FCurveEditorLatticeTool() override;

	//~ Begin ICurveEditorToolExtension Interface
	virtual void Tick(const FGeometry& InViewContainerGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual void OnPaint(const FPaintArgs& InPaintArgs, const FGeometry& InViewContainerGeometry, const FSlateRect& InMyCullingRect, FSlateWindowElementList& OutDrawElements, int32 InPaintOnLayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual FReply OnMouseButtonDoubleClick(TSharedRef<SWidget> OwningWidget, const FGeometry& InViewContainerGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnMouseButtonDown(TSharedRef<SWidget> InOwningWidget, const FGeometry& InViewContainerGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnMouseButtonUp(TSharedRef<SWidget> InOwningWidget, const FGeometry& InViewContainerGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(TSharedRef<SWidget> InOwningWidget, const FGeometry& InViewContainerGeometry, const FPointerEvent& InMouseEvent) override;
	virtual void OnFinishedPointerInput() override;
	virtual void OnToolActivated() override;
	virtual void OnToolDeactivated() override;
	virtual void BindCommands(TSharedRef<FUICommandList> CommandBindings) override;
	virtual FText GetLabel() const override;
	virtual FText GetDescription() const override;
	virtual FSlateIcon GetIcon() const override;
	//~ End ICurveEditorToolExtension Interface

	//~ Begin FGCObject Interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override {  return FString("FCurveEditorLatticeTool"); }
	//~ End FGCObject Interface

private:
	
	/** Weak pointer back to the owning curve editor. */
	TWeakPtr<FCurveEditor> WeakCurveEditor;
	
	/** Set when there is a valid selection in view. */
	TOptional<FLatticeDeformerState> LatticeDeformerState;
	
	/** How to draw LatticeDeformer. Set only when the selection, over which the controls are drawn, is visible.*/
	TOptional<FLatticeControlsDrawData> DrawData;

	/** The last mouse position received by OnMouseMove. */
	FVector2D FinalMouseScreenSpacePosition;
	/** Data used to detect dragging. */
	TOptional<FDelayedDragData> DelayedDragData;

	/**
	 * We store the shape of the lattice grid before starting the drag operation.
	 * This way, we can restore the shape of the grid when the user undo-es.
	 */
	TObjectPtr<UCurveEditorTools_LatticeUndoObject> UndoObject;

	/** Creates or destroys LatticeDeformer based on the selection. */
	void UpdateDeformerBounds(const TSharedRef<FCurveEditor>& InCurveEditor);

	/** Updates the bounds after the user has externally moved the keys, e.g. MMB move keys, etc. */
	void OnCurvesModified();
	/** Skips recomputing the bounds if the selection was changed due us modifying curves (e.g. because of stacked keys being removed) */
	void OnSelectionChanged();
	/** Regenerates the lattice. */
	void OnCurvePanelRebuilt() { RecomputeLatticeGrid(); }
	/**  */
	void OnTransactionStateChanged(const FTransactionContext& TransactionContext, ETransactionStateEventType TransactionStateEvent);
	
	/** Call this if the keys have externally moved. Tries to fit the lattice grid over the keys or reconstructs a new rectangular one if not possible. */
	void RecomputeLatticeGrid(const TSharedRef<FCurveEditor>& InCurveEditor, ELatticeUpdateFlags InRefreshFlags = ELatticeUpdateFlags::None);
	void RecomputeLatticeGrid(ELatticeUpdateFlags InRefreshFlags = ELatticeUpdateFlags::None);

	/** Recomputes all draw data. */
	void RecomputeDrawData(const FGeometry& InViewContainerGeometry, const TSharedPtr<FCurveEditor>& InCurveEditor);
	/** Populates the control points and edges. */
	TArray<FVector2D> ComputeDrawControlPoints(const FCurveEditor& InCurveEditor) const;
	TArray<FLatticeControlEdge> ComputeDrawControlEdges(TConstArrayView<FVector2D> InControlPoints) const;

	/** Stops the active drag operation. */
	void CancelDrag();
	/** Creates a drag-drop operation based on the hover state. */
	void StartDragDrop(const FGeometry& InViewContainerGeometry, const FPointerEvent& InMouseEvent);
	/** Starts a new drag drop operation based on whether a control point, edge, or cell is hovered. */
	TOptional<FDragOpData> CreateDragDrop(const FLatticeBounds& InBounds, const FLatticeHoverState& InHoverState);
	TOptional<FDragOpData> CreateDragDrop_Point(const FLatticeBounds& InBounds, int32 InControlPointIndex);
	TOptional<FDragOpData> CreateDragDrop_Edge(const FLatticeBounds& InBounds, int32 InEdgeIndex);
	TOptional<FDragOpData> CreateDragDrop_Cell(const FLatticeBounds& InBounds, int32 InCellIndex);

	// User actions: things the user can directly do in the UI.
	using FMovePointsSignature = void(FCurveEditorLatticeTool::*)(
		FPerCurveDeformer2D& InDeformer, FCurveModel& InCurveModel,
		TConstArrayView<int32> InVertexIndices, TConstArrayView<FVector2D> InNewControlPointPositions,
		EPropertyChangeType::Type InChangeType, EKeyMoveMode InSnapMode
		);
	/** Handles dragging a control point, edge, or the lattice center. */
	void HandleDragPoints(
		FMovePointsSignature InMovePointsCallback, TConstArrayView<int32> InVertexIndices, TConstArrayView<FVector2D> InNewControlPoints,
		EPropertyChangeType::Type InChangeType, EKeyMoveMode InSnapMode
		);
	/** Handles double-clicking an edge. */
	void HandleMoveEdgeToOpposite(int32 InEdgeIndex);
	/** Handles double-clicking the lattice center. */
	void HandleMirrorGrid();
	
	/** Called when a control point is dragged. */
	void OnMoveControlPoint(
		FPerCurveDeformer2D& InDeformer, FCurveModel& InCurveModel, TConstArrayView<int32> InVertexIndices, TConstArrayView<FVector2D> InNewControlPointPositions,
		EPropertyChangeType::Type InChangeType, EKeyMoveMode InSnapMode
		);
	/** Called when an edge is dragged. */
	void OnMoveEdge(FPerCurveDeformer2D& InDeformer, FCurveModel& InCurveModel,
		TConstArrayView<int32> InVertexIndices, TConstArrayView<FVector2D> InNewControlPointPositions,
		EPropertyChangeType::Type InChangeType, EKeyMoveMode InSnapMode
		);
	/** Generic function for updating control points. */
	void OnSetControlPoints(
		FPerCurveDeformer2D& InDeformer, FCurveModel& InCurveModel,
		TConstArrayView<int32> InControlPointsToUpdate, TConstArrayView<FVector2D> InNewControlPoints,
		EPropertyChangeType::Type InChangeType, EKeyMoveMode InSnapMode = EKeyMoveMode::Unrestricted
		);
	
	/** Moves the clicked edge to the opposite edge, effectively flattening the curve sections. */
	void MoveEdgeToOpposite(FPerCurveDeformer2D& InDeformer, FCurveModel& InCurveModel, FCurveEditor& InCurveEditor, int32 InEdgeIndex);
	/** Mirrors all points in the grid across an imaginary, x-axis aligned edge going through the grid center. Only if the grid consists of a single cell. */
	void MirrorGrid(FPerCurveDeformer2D& InDeformer, FCurveModel& InCurveModel, FCurveEditor& InCurveEditor);
	
	const FLatticeDeformer2D& GetGlobalDeformer() const { return LatticeDeformerState->GlobalDeformer; }
	FLatticeDeformer2D& GetGlobalDeformer() { return LatticeDeformerState->GlobalDeformer; }

	/**
	 * Invokes InCallback if there is a lattice grid shown and the user left clicks somewhere.
	 * @return False if InCallback was not invoked. Otherwise, whatever InCallback returned. */
	template<typename TCallback> requires std::is_invocable_r_v<bool, TCallback, const FLatticeHoverState&, const FLatticeBounds&, const FCurveEditor&>
	bool ComputeHoverStateForLeftClick(const FGeometry& InViewContainerGeometry, const FPointerEvent& InMouseEvent, TCallback&& InCallback);

	/** Shared logic for unregistering global delegates. */
	void UnregisterDelegates() const;

	/** Saves the undo state in the undo object. */
	void SaveUndoState(const FCurveEditor& InCurveEditor);
	bool ApplyUndoState(FCurveEditor& InCurveEditor);
};

inline void FCurveEditorLatticeTool::RecomputeLatticeGrid(ELatticeUpdateFlags InRefreshFlags)
{
	if (const TSharedPtr<FCurveEditor> CurveEditorPin = WeakCurveEditor.Pin())
	{
		RecomputeLatticeGrid(CurveEditorPin.ToSharedRef(), InRefreshFlags);
	}
}
}
