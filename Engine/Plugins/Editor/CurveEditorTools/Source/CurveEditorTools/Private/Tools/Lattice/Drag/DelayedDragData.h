// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragOpData.h"
#include "Framework/DelayedDrag.h"
#include "Misc/Optional.h"
#include "Tools/Lattice/LatticeControlsDrawData.h"
#include "Tools/Lattice/Misc/LatticeDrawUtils.h"

namespace UE::CurveEditorTools
{
/** Data used while the user is holding down the left mouse button. Once the mouse has moved enough, we "detect" a drag. */
struct FDelayedDragData
{
	/** Set when attempting to move a drag handle. This allows us to tell the difference between a click and a click-drag. */
	FDelayedDrag DelayedDrag;

	/** Bounds when the op was started. */
	const FLatticeBounds Bounds;
	/** Hover state when the op was started. */
	const FLatticeHoverState HoverState;

	/** Set if the drag up was started. */
	TOptional<FDragOpData> ActiveOperation;
		
	explicit FDelayedDragData(
		const FVector2D& InInitialPosition, const FKey& InEffectiveKey, const FLatticeBounds& InBounds, const FLatticeHoverState& InHoverState
		)
		: DelayedDrag(InInitialPosition, InEffectiveKey), Bounds(InBounds), HoverState(InHoverState)
	{}
};
}
