// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LatticeDragOp.h"
#include "Modification/Utils/ScopedCurveChange.h"
#include "ScopedTransaction.h"
#include "Templates/UniquePtr.h"
#include "Tools/Lattice/Mirror/LatticeEdgeTangentMirrorOp.h"
#include "Tools/Lattice/Mirror/LatticePointTangentMirrorOp.h"

namespace UE::CurveEditorTools
{
/** Data used after FDelayedDragData has detected a drag. The user is now actively dragging a control point, edge, or cell. */
struct FDragOpData
{
	/** Handles drags that started on a control point. */
	TUniquePtr<FLatticeDragOp> DragOp;
	/** This transaction is active for the duration of the drag. */
	TUniquePtr<FScopedTransaction> Transaction;
	/** Transactions the key changes */
	TUniquePtr<CurveEditor::FScopedCurveChange> KeyChange;
		
	/** Adjust tangents while dragging on an edge.*/
	TMap<FCurveModelID, FLatticeEdgeTangentsMirrorOp> EdgeTangentMirroringOps;
	/** Adjusts tangents while dragging on a control point. */
	TMap<FCurveModelID, FLatticePointTangentsMirrorOp> PointTangentMirroringOps;

	/** False until the first drag actually changes keys. We only need to capture undo state just before the first change. */
	bool bSavedUndoState = false;

	explicit FDragOpData(TWeakPtr<FCurveEditor> InCurveEditor, TUniquePtr<FLatticeDragOp> DragOp, const FText& TransactionText)
		: DragOp(MoveTemp(DragOp))
		, Transaction(MakeUnique<FScopedTransaction>(TEXT("CurveEditorLatticeTool"), TransactionText, nullptr))
		, KeyChange([InCurveEditor]
		{
			using namespace UE::CurveEditor;
			return MakeUnique<FScopedCurveChange>(
				FCurvesSnapshotBuilder(
					InCurveEditor,
					ECurveChangeFlags::MoveKeysAndRemoveStackedKeys | ECurveChangeFlags::KeyAttributes
					).TrackSelectedCurves()
				);
		}())
	{}
};
}
