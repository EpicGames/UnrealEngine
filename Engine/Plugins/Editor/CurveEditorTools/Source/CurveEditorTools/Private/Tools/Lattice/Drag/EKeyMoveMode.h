// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"

namespace UE::CurveEditorTools
{
/** Describes how keys are supposed to be snapped when using the lattice tool. */
enum class EKeyMoveMode : uint8
{
	/** Mouse is unlocked, i.e can move freely in 2D space. */
	Unrestricted,
	
	/** Do not move the keys in X direction. E.g. when user has shift pressed and is moving up / down (in y direction). */
	IgnoreX,
	/** Do not move the keys in Y direction. E.g. when user has shift pressed and is moving left / right (in x direction). */
	IgnoreY,

	IgnoreAll = IgnoreY | IgnoreX
};
ENUM_CLASS_FLAGS(EKeyMoveMode);
}