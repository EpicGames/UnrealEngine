// Copyright Epic Games, Inc. All Rights Reserved.

#include "LatticeDragOp.h"

#include "CurveEditor.h"

namespace UE::CurveEditorTools
{
FLatticeDragOp::FLatticeDragOp(TWeakPtr<FCurveEditor> InCurveEditor)
	: CurveEditor(MoveTemp(InCurveEditor))
	, InitialMousePosition()
{}

void FLatticeDragOp::BeginDrag(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent, const FVector2D& InInitialMousePosition)
{
	InitialMousePosition = LastMousePosition = InMouseEvent.GetScreenSpacePosition();
	AccumulateMouseMovement(InGeometry, InMouseEvent);
	OnBeginDrag(InGeometry, InitialMousePosition, AccumulatedMouseMovement->LastDragMode);
}

void FLatticeDragOp::MoveMouse(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	AccumulateMouseMovement(InGeometry, InMouseEvent);
}

void FLatticeDragOp::FinishedPointerInput()
{
	if (AccumulatedMouseMovement)
	{
		OnMoveMouse(AccumulatedMouseMovement->CachedGeometry, AccumulatedMouseMovement->AccumulatedPosition, AccumulatedMouseMovement->LastDragMode);
		AccumulatedMouseMovement.Reset();
	}
}

void FLatticeDragOp::EndDrag(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	// Snap the mouse position according to the axis snapping settings.
	AccumulateMouseMovement(InGeometry, InMouseEvent);
	OnEndDrag(InGeometry, AccumulatedMouseMovement->AccumulatedPosition, AccumulatedMouseMovement->LastDragMode);
	AccumulatedMouseMovement.Reset();
}

void FLatticeDragOp::CancelDrag()
{
	OnCancelDrag();
}

namespace Private
{
static EKeyMoveMode DetermineSnapMode(const FCurveEditorAxisSnap& Snap, const FCurveEditorAxisSnap::FSnapState& SnapState)
{
	const bool bIsRestrictingWithShift = SnapState.bHasPassedThreshold;
	if (bIsRestrictingWithShift)
	{
		return FMath::IsNearlyZero(SnapState.MouseLockVector.X) ? EKeyMoveMode::IgnoreX : EKeyMoveMode::IgnoreY;
	}

	// This handles the start of the drag: shift is pressed but not yet above the threshold.
	// Without this, all keys would get snapped and then become unsnapped once the mouse goes over the threshold. It's a visual unpleasantness.
	if (SnapState.bHasStartPosition)
	{
		return EKeyMoveMode::IgnoreAll;
	}
	
	switch (Snap.RestrictedAxisList)
	{
	case ECurveEditorSnapAxis::CESA_None: return EKeyMoveMode::Unrestricted;
	case ECurveEditorSnapAxis::CESA_X: return EKeyMoveMode::IgnoreY;
	case ECurveEditorSnapAxis::CESA_Y: return EKeyMoveMode::IgnoreX;

	default:
		return EKeyMoveMode::Unrestricted;
	}
}
}

void FLatticeDragOp::AccumulateMouseMovement(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	const TSharedPtr<FCurveEditor> CurveEditorPin = CurveEditor.Pin();
	if (!CurveEditorPin)
	{
		return;
	}
	
	const FVector2D ScreenSpacePosition = InMouseEvent.GetScreenSpacePosition();
	LastMousePosition = ScreenSpacePosition;
	
	FCurveEditorAxisSnap Snap = CurveEditorPin->GetAxisSnap();
	const FVector2D SnappedPosition = Snap.GetSnappedPosition(
		InitialMousePosition, ScreenSpacePosition, LastMousePosition, InMouseEvent, SnapState
		);
	const EKeyMoveMode SnapMode = Private::DetermineSnapMode(Snap, SnapState);
	
	if (!AccumulatedMouseMovement)
	{
		AccumulatedMouseMovement.Emplace(InGeometry, SnappedPosition, SnapMode);
	}
	else
	{
		AccumulatedMouseMovement->AccumulatedPosition = SnappedPosition;
		AccumulatedMouseMovement->LastDragMode = SnapMode;
	}
}
}
