// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditorSnapMetrics.h"
#include "EKeyMoveMode.h"
#include "Layout/Geometry.h"
#include "Math/MathFwd.h"
#include "Math/Vector2D.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"

class FCurveEditor;

namespace UE::CurveEditorTools
{
/** Handles updating the lattice deformer's control point(s) in response to a drag. */
class FLatticeDragOp
{
public:

	explicit FLatticeDragOp(TWeakPtr<FCurveEditor> InCurveEditor);

	void BeginDrag(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent, const FVector2D& InInitialMousePosition);
	void MoveMouse(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent);
	void FinishedPointerInput();
	void EndDrag(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent);
	void CancelDrag();

	virtual ~FLatticeDragOp() = default;

protected:

	virtual void OnBeginDrag(const FGeometry& InGeometry, const FVector2D& InInitialMousePosition, EKeyMoveMode InSnapMode) {}
	virtual void OnMoveMouse(const FGeometry& InGeometry, const FVector2D& InScreenPosition, EKeyMoveMode InSnapMode) {}
	virtual void OnEndDrag(const FGeometry& InGeometry, const FVector2D& InMousePosition, EKeyMoveMode InSnapMode) {}
	virtual void OnCancelDrag() {}

	TWeakPtr<FCurveEditor> GetCurveEditor() const { return CurveEditor; }
	FVector2D GetInitialMousePosition() const { return InitialMousePosition; }
	
private:

	/** Used to get snapping settings. */
	const TWeakPtr<FCurveEditor> CurveEditor;
	
	/** Set by OnBeginDrag. */
	FVector2D InitialMousePosition;
	/** Used by the snap logic. Set every time mouse moved. */
	FVector2D LastMousePosition;

	/** Passed to the snap logic */
	FCurveEditorAxisSnap::FSnapState SnapState;
	
	struct FAccumulatedMouseMovement
	{
		const FGeometry CachedGeometry;
		
		/** The mouse position that has been accumulated by processing all pumped input this tick. */
		FVector2D AccumulatedPosition;
		/** The mouse lock mode that has been accumulated by processing all pumped input this tick. */
		EKeyMoveMode LastDragMode;

		explicit FAccumulatedMouseMovement(const FGeometry& InCachedGeometry, const FVector2D& InInitialPosition, EKeyMoveMode InInitialDragMode)
			: CachedGeometry(InCachedGeometry)
			, AccumulatedPosition(InInitialPosition)
			, LastDragMode(InInitialDragMode)
		{}
	};
	TOptional<FAccumulatedMouseMovement> AccumulatedMouseMovement;

	void AccumulateMouseMovement(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent);
};
}

