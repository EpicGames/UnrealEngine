// Copyright Epic Games, Inc. All Rights Reserved.

#include "LatticeDragOp_MoveControlPoints.h"

#include "Templates/SharedPointer.h"
#include "Tools/Lattice/Misc/LatticeDrawUtils.h"

namespace UE::CurveEditorTools
{
FLatticeDragOp_MoveControlPoints::FLatticeDragOp_MoveControlPoints(
	TWeakPtr<FCurveEditor> InCurveEditor,
	FControlPointArray InInitialControlPoints,
	const FLatticeBounds& InBounds,
	FOnBeginDragOperation InOnBeginDragControlPointsDelegate,
	FOnControlPointsMoved InInteractiveDragControlPointsDelegate,
	FOnControlPointsMoved InFinishDragControlPointsDelegate
	)
	: FLatticeDragOp(MoveTemp(InCurveEditor))
	, InitialControlPoints(MoveTemp(InInitialControlPoints))
	, Bounds(InBounds)
	, OnBeginDragControlPointsDelegate(MoveTemp(InOnBeginDragControlPointsDelegate))
	, OnInteractiveDragControlPointsDelegate(MoveTemp(InInteractiveDragControlPointsDelegate))
	, OnFinishDragControlPointsDelegate(MoveTemp(InFinishDragControlPointsDelegate))
	, UpdatedControlPointPositions(InitialControlPoints)
{}

void FLatticeDragOp_MoveControlPoints::OnBeginDrag(const FGeometry& InGeometry, const FVector2D& InInitialMousePosition, EKeyMoveMode InSnapMode)
{
	OnBeginDragControlPointsDelegate.ExecuteIfBound();
}

void FLatticeDragOp_MoveControlPoints::OnMoveMouse(const FGeometry& InGeometry, const FVector2D& InScreenPosition, EKeyMoveMode InSnapMode)
{
	UpdateControlPoints(InGeometry, InScreenPosition);
	OnInteractiveDragControlPointsDelegate.Execute(UpdatedControlPointPositions, InSnapMode);
}

void FLatticeDragOp_MoveControlPoints::OnEndDrag(const FGeometry& InGeometry, const FVector2D& InMousePosition, EKeyMoveMode InSnapMode)
{
	UpdateControlPoints(InGeometry, InMousePosition);
	OnFinishDragControlPointsDelegate.Execute(UpdatedControlPointPositions, InSnapMode);
}

void FLatticeDragOp_MoveControlPoints::UpdateControlPoints(const FGeometry& InGeometry, const FVector2D& InScreenPosition)
{
	const FSlateLayoutTransform ScreenToWidget = InGeometry.GetAccumulatedLayoutTransform().Inverse();
	const FVector2D Initial_WidgetSpace = ScreenToWidget.TransformPoint(GetInitialMousePosition());
	const FVector2D New_WidgetSpace = ScreenToWidget.TransformPoint(InScreenPosition);
		
	const FVector2D DeltaMove_WidgetSpace = New_WidgetSpace - Initial_WidgetSpace;
	FVector2D ValuePerSlate = (Bounds.MaxValues - Bounds.MinValues) / (Bounds.MaxSlatePosition - Bounds.MinSlatePosition);
	ValuePerSlate.Y *= -1.f; // Mouse.Y and Screen.Y move in opposite directions.
	const FVector2D DeltaMove_ControlPointSpace = DeltaMove_WidgetSpace * ValuePerSlate;

	for (int32 Index = 0; Index < InitialControlPoints.Num(); ++Index)
	{
		UpdatedControlPointPositions[Index] = InitialControlPoints[Index] + DeltaMove_ControlPointSpace;
	}
}
}
