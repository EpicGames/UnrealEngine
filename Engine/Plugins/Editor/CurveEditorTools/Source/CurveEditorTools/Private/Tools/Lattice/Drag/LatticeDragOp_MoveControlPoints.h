// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "LatticeDragOp.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointerFwd.h"
#include "Tools/Lattice/Misc/LatticeDrawUtils.h"

class FCurveEditor;

namespace UE::CurveEditorTools
{
struct FLatticeBounds;
using FControlPointArray = TArray<FVector2D, TInlineAllocator<4>>;
	
/** Given an array of control points, invokes a delegate with the update control point locations based on the mouse movement. */
class FLatticeDragOp_MoveControlPoints : public FLatticeDragOp
{
public:

	using FOnBeginDragOperation = FSimpleDelegate;
	DECLARE_DELEGATE_TwoParams(FOnControlPointsMoved, const TConstArrayView<FVector2D>& InNewControlPointPositions, EKeyMoveMode InSnapMode);

	explicit FLatticeDragOp_MoveControlPoints(
		TWeakPtr<FCurveEditor> InCurveEditor,
		FControlPointArray InInitialControlPoints,
		const FLatticeBounds& InBounds,
		FOnBeginDragOperation InOnBeginDragControlPointsDelegate,
		FOnControlPointsMoved InInteractiveDragControlPointsDelegate,
		FOnControlPointsMoved InFinishDragControlPointsDelegate
		);

protected:
	
	//~ Begin FLatticeDragOp Interface
	virtual void OnBeginDrag(const FGeometry& InGeometry, const FVector2D& InInitialMousePosition, EKeyMoveMode InSnapMode) override;
	virtual void OnMoveMouse(const FGeometry& InGeometry, const FVector2D& InScreenPosition, EKeyMoveMode InSnapMode) override;
	virtual void OnEndDrag(const FGeometry& InGeometry, const FVector2D& InMousePosition, EKeyMoveMode InSnapMode) override;
	//~ End FLatticeDragOp Interface

private:

	/** Initial positions of the control points. */
	const FControlPointArray InitialControlPoints;
	/** Used to transform points */
	const FLatticeBounds Bounds;

	/** Invoked when the drag operation begins. A call to OnFinishDragControlPointsDelegate will follow some time in the future. */
	const FOnBeginDragOperation OnBeginDragControlPointsDelegate;
	/** Invoked while the mouse button is down. */
	const FOnControlPointsMoved OnInteractiveDragControlPointsDelegate;
	/** Invoked one last time when the mouse button is lifted. */
	const FOnControlPointsMoved OnFinishDragControlPointsDelegate;

	/** Updated every time the mouse moves: InitialControlPoints + DeltaMouseMove. */
	FControlPointArray UpdatedControlPointPositions;

	/** Updates UpdatedControlPointPositions. */
	void UpdateControlPoints(const FGeometry& InGeometry, const FVector2D& InScreenPosition);
};
}

