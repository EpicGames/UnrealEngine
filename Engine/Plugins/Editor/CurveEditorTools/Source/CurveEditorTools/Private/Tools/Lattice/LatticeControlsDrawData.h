// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Math/MathFwd.h"
#include "Math/IntPoint.h"
#include "Math/Vector2D.h"
#include "Misc/Optional.h"
#include "HAL/Platform.h"

namespace UE::CurveEditorTools
{
struct FLatticeControlEdge;

/** Describes what is hovered in the control widget. */
struct FLatticeHoverState
{
	/** Index to ControlPointMatrix. Indicates hovered control point. */
	TOptional<int32> HoveredControlPoint;
	/**
	 * Index to edge.
	 * @see Indexing of edges in FLatticeDeformer2D.
	 */
	TOptional<int32> HoveredEdge;

	/**
	 * Index to the cell.
	 * @see Indexing of cells in FLatticeDeformer2D. 
	 */
	TOptional<int32> HoveredCell;

	bool IsHovered() const { return HoveredControlPoint || HoveredEdge || HoveredCell; }
};
	
/**
 * Holds the data required to draw a lattice control widget.
 *
 * This struct represents the exact information the drawing algorithm requires. The intention is that the drawing thus becomes as straighforward
 * as possible. This way, if we wanted to, we could unit test the produced FLatticeControlsDrawData.
 */
struct FLatticeControlsDrawData
{
	/** The control points in the space of the SCurveEditorViewContainer (i.e. what's passed into ICurveEditorToolExtension::OnPaint). */
	TArray<FVector2D> ControlPoints;
	/** The edges to draw */
	TArray<FLatticeControlEdge> ControlEdges;

	/** Number of points in x-direction. Correlates with ControlPoints.Num(). */
	int32 MatrixWidth;

	/** Determines which index in ControlPoints or ControlEdges should be drawn as hovered. */
	FLatticeHoverState HoverState;
};
}

