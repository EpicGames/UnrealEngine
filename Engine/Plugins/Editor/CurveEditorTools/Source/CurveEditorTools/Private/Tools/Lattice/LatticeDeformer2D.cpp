// Copyright Epic Games, Inc. All Rights Reserved.

#include "LatticeDeformer2D.h"

#include "Algo/AllOf.h"
#include "Async/ParallelFor.h"
#include "Misc/LatticeUtils.h"
#include "HAL/Platform.h"

namespace UE::CurveEditorTools::Math
{
/** @return The UV coordinates InPoint has in the square formed by InBottomLeft and InTopRight. */
static FVector2D ComputeCoordinatesInCell(const FVector2D& InPoint, const FVector2D& InBottomLeft, const FVector2D& InTopRight)
{
	const double MinX = InBottomLeft.X;
	const double MinY = InBottomLeft.Y;
	const double Width = InTopRight.X - MinX;
	const double Height = InTopRight.Y - MinY;

	// Division by 0 should not occur because we check for 0 width and height in constructor.
	// However, maybe a future change breaks this assumption (fix it!), or bad API usage calls UpdateControlPoints before AddPoints_BeforeLatticeMove.
	const bool bNonZeroWidth = !FMath::IsNearlyZero(Width);
	const bool bNonZeroHeight =!FMath::IsNearlyZero(Height);
	checkSlow(bNonZeroWidth && bNonZeroHeight);
	
	const double U = LIKELY(bNonZeroWidth) ? (InPoint.X - MinX) / Width : 0.0;
	const double V = LIKELY(bNonZeroHeight) ? (InPoint.Y - MinY) / Height : 0.0;
	
	return { U, V };
}
}

namespace UE::CurveEditorTools
{
FLatticeDeformer2D::FLatticeDeformer2D(int32 InNumCellsInX, int32 InNumCellsInY, const FVector2D& InBottomLeft, const FVector2D& InTopRight)
	: CellDimensions(InNumCellsInX, InNumCellsInY)
	, ControlPoints(GenerateControlPoints(InNumCellsInX, InNumCellsInY, InBottomLeft, InTopRight))
{
	if (!ControlPoints.IsEmpty())
	{
		Cells.SetNum(NumCells());
	}
}

void FLatticeDeformer2D::UpdateControlPoints(
	TConstArrayView<int32> InControlPointsToUpdate, TConstArrayView<FVector2D> InNewControlPoints,
	const TFunctionRef<void(const FPointIndex&, const FVector2D&)> HandleKeyChange
	)
{
	const bool bValidIndices = Algo::AllOf(InControlPointsToUpdate, [this](int32 InIndex){ return ControlPoints.IsValidIndex(InIndex); });
	if (!ensure(bValidIndices && InControlPointsToUpdate.Num() == InNewControlPoints.Num())
		|| InControlPointsToUpdate.IsEmpty())
	{
		return;
	}
	
	if (NumCells() == 0)
	{
		SetControlsPointNoRecompute(InControlPointsToUpdate, InNewControlPoints);
		
		// Fast path for common 1x1 lattice case
		RecomputeCell(0, HandleKeyChange);
	}
	else
	{
		TArray<int32> AffectedCells;
		for (int32 Index = 0; Index < InControlPointsToUpdate.Num(); ++Index)
		{
			const int32 ControlPointIndex = InControlPointsToUpdate[Index];
			ControlPoints[ControlPointIndex] = InNewControlPoints[Index];

			// Get the cell indices of the cell to the top-left, etc. of the corner.
			const auto[TopLeft, TopRight, BottomRight, BottomLeft] = GetMatrixCellSlotsWithCorner(
				ControlPointIndex, NumPointsInWidth(), NumPointsInHeight()
				);
			// Need to recompute every cell that shares corner with the control point.
			AffectedCells.AddUnique(TopLeft);
			AffectedCells.AddUnique(TopRight);
			AffectedCells.AddUnique(BottomRight);
			AffectedCells.AddUnique(BottomLeft);
		}

		for (const int32& CellIndex : AffectedCells)
		{
			RecomputeCell(CellIndex, HandleKeyChange);
		}
	}
}

void FLatticeDeformer2D::SetControlsPointNoRecompute(TConstArrayView<int32> InControlPointsToUpdate, TConstArrayView<FVector2D> InNewControlPoints)
{
	const bool bValidIndices = Algo::AllOf(InControlPointsToUpdate, [this](int32 InIndex){ return ControlPoints.IsValidIndex(InIndex); });
	if (!ensure(bValidIndices && InControlPointsToUpdate.Num() == InNewControlPoints.Num())
		|| InControlPointsToUpdate.IsEmpty())
	{
		return;
	}
	
	for (int32 Index = 0; Index < InControlPointsToUpdate.Num(); ++Index)
	{
		ControlPoints[InControlPointsToUpdate[Index]] = InNewControlPoints[Index];
	}
}

FLatticeControlEdge FLatticeDeformer2D::GetControlEdge(int32 InEdgeIndex) const
{
	const auto[Start, End] = GetEdgeIndices(InEdgeIndex);
	if (ensure(ControlPoints.IsValidIndex(Start) && ControlPoints.IsValidIndex(End)))
	{
		return FLatticeControlEdge(ControlPoints[Start], ControlPoints[End]);
	}
	return FLatticeControlEdge();
}

FEdgeVertexIndices FLatticeDeformer2D::GetEdgeIndices(int32 InEdgeIndex) const
{
	const int32 Width = NumPointsInWidth();
	const int32 Height = NumPointsInHeight();
	const bool bIsInRange = Width > 0 && Height > 0 && InEdgeIndex >= 0 && InEdgeIndex < NumEdges();
	return ensure(bIsInRange) ? CurveEditorTools::GetEdgeIndices(InEdgeIndex, Width, Height) : FEdgeVertexIndices();
}

FCellVertexIndices FLatticeDeformer2D::GetCellIndices(int32 InCellIndex) const
{
	const bool bIsInRange = InCellIndex >= 0 && InCellIndex < NumCells();
	return ensure(bIsInRange) ? CurveEditorTools::GetCellIndices(InCellIndex, NumPointsInWidth()) : FCellVertexIndices();
}

void FLatticeDeformer2D::SetKeyCoordsInCell(int32 InCellIndex, TArray<FLatticeKeyCoords> InNewKeyCoords)
{
	if (ensure(Cells.IsValidIndex(InCellIndex)))
	{
		Cells[InCellIndex].KeyCoords = MoveTemp(InNewKeyCoords);
	}
}

TConstArrayView<FLatticeKeyCoords> FLatticeDeformer2D::GetKeyCoordsInCell(int32 InCellIndex) const
{
	return ensure(Cells.IsValidIndex(InCellIndex)) ? Cells[InCellIndex].KeyCoords : TConstArrayView<FLatticeKeyCoords>();
}

int32 FLatticeDeformer2D::FindCellContainingPoint(const FVector2D& InPoint) const
{
	if (NumCells() <= 1)
	{
		// Fast-path
		return 0; // If there are 0 cells, Cells.IsValidIndex(0) will still return false
	}
	else
	{
		const FVector2D& BottomLeft = ControlPoints[MatrixIndicesToFlatIndex(0, NumPointsInHeight() - 1, NumPointsInWidth())];
		const FVector2D& TopRight = ControlPoints[NumPointsInWidth() - 1];
		const FVector2D& UV = Math::ComputeCoordinatesInCell(InPoint, BottomLeft, TopRight);
		const bool bIsInGrid = 0.0 <= UV.GetMin() && UV.GetMax() <= 1.0;
		if (!bIsInGrid)
		{
			return INDEX_NONE;
		}
		
		const int32 Column = static_cast<int32>(UV.X / NumPointsInWidth());
		const int32 Row = static_cast<int32>(UV.Y / NumPointsInHeight());
		return MatrixIndicesToFlatIndex(Column, Row, NumCellsInWidth());
	}
}

int32 FLatticeDeformer2D::NumKeys() const
{
	int32 Count = 0;
	for (int32 CellIndex = 0; CellIndex < Cells.Num(); ++CellIndex)
	{
		Count += Cells[CellIndex].KeyCoords.Num();
	}
	return Count;
}

int32 FLatticeDeformer2D::NumEdges() const
{
	if (NumPointsInWidth() == 0 || NumPointsInHeight() == 0)
	{
		return 0; 
	}

	const int32 HorizontalEdges = (NumPointsInWidth() - 1) * NumPointsInHeight();
	const int32 VerticalEdges = (NumPointsInHeight() - 1) * NumPointsInWidth();

	// Total number of edges
	return HorizontalEdges + VerticalEdges;
}

int32 FLatticeDeformer2D::NumCells() const
{
	return (NumPointsInWidth() - 1) * (NumPointsInHeight() - 1);
}
	
void FLatticeDeformer2D::AddPoints_BeforeLatticeMoved(
	TConstArrayView<FVector2D> InPoints,
	TFunctionRef<void(int32 InputIndex, const FPointIndex&)> OnPointIndexed
	)
{
	// No need to make this ParallelFor: 4500 keys take about 0.6ms.
	for (int32 PointIndex = 0; PointIndex < InPoints.Num(); ++PointIndex)
	{
		const FVector2D& KeyPoint = InPoints[PointIndex];
		// Checking whether the point is in the cell is relatively cheap (addition & multiplication) ...
		const int32 CellIndex = FindCellContainingPoint(KeyPoint);
		if (!Cells.IsValidIndex(CellIndex))
		{
			continue;
		}
		
		TArray<FLatticeKeyCoords>& Keys = Cells[CellIndex].KeyCoords;
		const int32 ItemIndex = Keys.Emplace(ComputeSingleCellWeights(GetCellIndices(CellIndex), InPoints[PointIndex]));
		OnPointIndexed(PointIndex, FPointIndex(CellIndex, ItemIndex));
	}
}

FLatticeKeyCoords FLatticeDeformer2D::ComputeSingleCellWeights(const FCellVertexIndices& InIndices, const FVector2D& InPoint) const
{
	const auto[TopLeft, TopRight, BottomRight, BottomLeft] = InIndices;
	const FVector2D Vertex_TopRight = ControlPoints[TopRight];
	const FVector2D Vertex_BottomLeft = ControlPoints[BottomLeft];
	return Math::ComputeCoordinatesInCell(InPoint, Vertex_BottomLeft, Vertex_TopRight);
}

void FLatticeDeformer2D::RecomputeCell(
	int32 InCellIndex, const TFunctionRef<void(const FPointIndex&, const FVector2D&)> HandleKeyChange
	)
{
	if (!Cells.IsValidIndex(InCellIndex))
	{
		return;
	}
		
	const FCellVertexIndices CellVertices = GetCellIndices(InCellIndex);

	// This takes about 0.083 ms for 9000 keys.
	TArray<FLatticeKeyCoords>& Keys = Cells[InCellIndex].KeyCoords;
	TArray<FVector2D> Positions;
	Positions.SetNumUninitialized(Keys.Num());
	ParallelFor(Keys.Num(), [this, &CellVertices, &Keys, &Positions](int32 KeyIndex)
	{
		FLatticeKeyCoords& KeyData = Keys[KeyIndex];
		Positions[KeyIndex] = RecomputeKeyValue(KeyData, CellVertices);
	});

	for (int32 Index = 0; Index < Keys.Num(); ++Index)
	{
		HandleKeyChange(FPointIndex(InCellIndex, Index), Positions[Index]);
	}
}

FVector2D FLatticeDeformer2D::RecomputeKeyValue(const FLatticeKeyCoords& PointCoords, const FCellVertexIndices& InIndices) const
{
	const FVector2D Inverse(1.0 - PointCoords.X, 1.0 - PointCoords.Y);

	// Compute the new point based on the coordinates it had in the original lattice...
	// FinalPoint = (1-U)*(1-V)*BottomLeft + (1-U)*V*TopLeft + U*(1-V)*BottomRight + U*V*TopRight
	const FVector2D BottomLeft = Inverse.X * Inverse.Y * ControlPoints[InIndices.BottomLeft];
	const FVector2D TopLeft = Inverse.X * PointCoords.Y * ControlPoints[InIndices.TopLeft];
	const FVector2D BottomRight = PointCoords.X * Inverse.Y * ControlPoints[InIndices.BottomRight];
	const FVector2D TopRight = PointCoords.X * PointCoords.Y * ControlPoints[InIndices.TopRight];

	const FVector2D FinalPoint = BottomLeft + TopLeft + BottomRight + TopRight;
	return { FinalPoint.X, FinalPoint.Y };
}
}