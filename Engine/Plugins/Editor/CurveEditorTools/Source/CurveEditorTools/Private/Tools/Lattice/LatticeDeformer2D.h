// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "HAL/Platform.h"
#include "Math/MathFwd.h"
#include "Math/Vector2D.h"
#include "Templates/FunctionFwd.h"

namespace UE::CurveEditorTools
{
struct FEdgeVertexIndices;
struct FCellVertexIndices;
	
/**  */
struct FLatticeControlEdge
{
	/** This allows us to convert TArray<FLatticeControlEdge> to TArray<FVector2D> by just taking the pointer to Points. */
	FVector2D Points[2];

	const FVector2D& Start() const { return Points[0]; }
	const FVector2D& End() const { return Points[1]; }
	
	FVector2D& Start() { return Points[0]; }
	FVector2D& End() { return Points[1]; }

	explicit FLatticeControlEdge() : FLatticeControlEdge(FVector2D::ZeroVector, FVector2D::ZeroVector) {}
	explicit FLatticeControlEdge(const FVector2D& Start, const FVector2D& End)
		: Points{ Start, End }
	{}
};

/** Describes how to locate a key in FLatticeDeformer2D */
struct FPointIndex
{
	/** Index of the cell in the lattice. */
	int32 CellIndex;
	/** Index in the cell's key array. */
	int32 IndexInCell;

	explicit FPointIndex(int32 InCellIndex, int32 InIndexInCell) : CellIndex(InCellIndex), IndexInCell(InIndexInCell) {}
};

/** Identifies the normalized coordinates a key has in a cell. */
using FLatticeKeyCoords = FVector2D;
	
/**
 * Implements a lattice deformer for 2D operations.
 *
 * ===== Indexing ====
 * Control points:
 *  0---1---2---3
 *	|   |   |   |
 *	4---5---6---7
 *	|   |   |   |
 *	8---9--10--11
 * 
 * Below, x are control points and - / I are edges.
 * Edge indices are numbered by rows then by columns, i.e.
 *  x--0--x--1--x
 *  6     7     8
 *  x--2--x--3--x  
 *  9     10    11
 *  x--4--x--5--x 
 *
 * Cells indices are numbered from top-left to bottom-right, i.e.
 * x---x---x
 * I 0 I 1 I
 * x---x---x
 *
 * In the future, if FLatticeDeformer2D needs to be generalized / extracted from this module, FKeyInCurveId can become a templated parameter since
 * FLatticeDeformer2D does not directly depend on it.
 */
class FLatticeDeformer2D
{
public:

	/** Completely resets this deformer and sets a new grid.  */
	explicit FLatticeDeformer2D(int32 InNumCellsInX, int32 InNumCellsInY, const FVector2D& InBottomLeft, const FVector2D& InTopRight);
	
	/** Moves the specified control points and recomputes the affected points. */
	void UpdateControlPoints(
		TConstArrayView<int32> InControlPointsToUpdate, TConstArrayView<FVector2D> InNewControlPoints,
		const TFunctionRef<void(const FPointIndex&, const FVector2D&)> HandleKeyChange
		);

	/** Sets the value of the control points without recomputing the values of the keys. Useful if you are resetting to a previous state. */
	void SetControlsPointNoRecompute(TConstArrayView<int32> InControlPointsToUpdate, TConstArrayView<FVector2D> InNewControlPoints);

	/** @return The control points, which is a flattened matrix. */
	TConstArrayView<FVector2D> GetControlPoints() const { return ControlPoints; }
	/** @param InIndex 0 <= InIndex < GetNumEdges() */
	FLatticeControlEdge GetControlEdge(int32 InEdgeIndex) const;
	
	/**
	 * @param InEdgeIndex 0 <= InEdgeIndex < NumEdges().
	 * @return Indices to GetControlPoints or all INDEX_NONE if invalid.
	 */
	FEdgeVertexIndices GetEdgeIndices(int32 InEdgeIndex) const;
	/**
	 * @param InCellIndex 0 <= InCellIndex < NumCells()
	 * @return Indices to GetControlPoints or all INDEX_NONE if invalid.
	 */
	FCellVertexIndices GetCellIndices(int32 InCellIndex) const;

	/** Sets the key coords of keys in the cell. This is useful for restoring previous data. */
	void SetKeyCoordsInCell(int32 InCellIndex, TArray<FLatticeKeyCoords> InNewKeyCoords);
	/** @return The coordinates of the keys in the cell. */
	TConstArrayView<FLatticeKeyCoords> GetKeyCoordsInCell(int32 InCellIndex) const;
	
	/** @return The vertices of the cell containing InPoint, or all INDEX_NONE if not contained.  */
	int32 FindCellContainingPoint(const FVector2D& InPoint) const;

	/** @return Counts the number of keys stored in all cells. */
	int32 NumKeys() const;
	
	int32 NumControlPoints() const { return ControlPoints.Num(); }
	int32 NumPointsInWidth() const { return CellDimensions.X + 1; }
	int32 NumPointsInHeight() const { return CellDimensions.Y + 1; }
	int32 NumEdges() const;
	int32 NumCellsInWidth() const { return CellDimensions.X; }
	int32 NumCellsInHeight() const { return CellDimensions.Y; }
	int32 NumCells() const;

protected:
	
	/**
	 * Adds the given points to the lattice grid.
	 * 
	 * This function assumes that none of the control points have yet been moved: if so, the computed weights are incorrect.
	 * The computation of the weights requires that all cells are rectangular.
	 *
	 * Exposed only to subclasses so subclasses can control how points are added.
	 * If this was public, you could downcast sub-classes to this base class and break the Liskov-Substitution principle.
	 */
	void AddPoints_BeforeLatticeMoved(
		TConstArrayView<FVector2D> InPoints,
		TFunctionRef<void(int32 InputIndex, const FPointIndex&)> OnPointIndexed = [](auto, auto){}
		);
	
	/** Computes the weights for InKeyData such that only the corners of the cell it is in affect it. */
	FLatticeKeyCoords ComputeSingleCellWeights(const FCellVertexIndices& InIndices, const FVector2D& InPoint) const;
	
private:
	
	/** The number of cells in X and Y direction. */
	const FIntPoint CellDimensions;
	
	/**
	 * The control points of the deformer. Moving a control point causes the keys it affects to change transform as well.
	 * 
	 * This is a flat matrix of dimension CellDimensions.X + 1 * CellDimensions.Y + 1,
	 * i.e. the 1st point in the 2nd row would be ControlPointMatrix[CellDimensions.X + 2]
	 */
	TArray<FVector2D> ControlPoints;

	struct FCellData
	{
		TArray<FLatticeKeyCoords> KeyCoords;
	};
	/**
	 * Caches which points belong to which cells. Moving a control point causes the keys it affects to change transform as well.
	 * 
	 * This is a flat matrix of dimension CellDimensions.X * CellDimensions.Y ,
	 * i.e. the 1st point in the 2nd row would be ControlPointMatrix[CellDimensions.X + 1]
	 */
	TArray<FCellData> Cells;

	/** Recomputes all key positions in the cell. Call this when cell has changed its control points. */
	void RecomputeCell(int32 InCellIndex, const TFunctionRef<void(const FPointIndex&, const FVector2D&)> HandleKeyChange);

	/** Computes the value the key should by computing the weighted linear product of all control points affecting it. */
	FVector2D RecomputeKeyValue(const FLatticeKeyCoords& InPointCoords, const FCellVertexIndices& InIndices) const;
};

/** Allows you to attach meta-data to each key position. */
template<typename TPointMetaData>
class TLatticeDeformer2D : public FLatticeDeformer2D
{
	using Super = FLatticeDeformer2D;
	
public:

	/** Completely resets this deformer and sets a new grid.  */
	explicit TLatticeDeformer2D(int32 InNumCellsInX, int32 InNumCellsInY, const FVector2D& InBottomLeft, const FVector2D& InTopRight)
		: Super(InNumCellsInX, InNumCellsInY, InBottomLeft, InTopRight)
	{
		CellMetaData.SetNum(NumCells());
	}

	/**
	 * Computes the weights for each of the points.
	 * 
	 * This function assumes that none of the control points have yet been moved: if so, the computed weights are incorrect.
	 * The computation of the weights requires that all cells are rectangular.
	 */
	void AddPoints_BeforeLatticeMoved(TConstArrayView<TPointMetaData> InKeys, TConstArrayView<FVector2D> InPoints)
	{
		if (!ensure(InKeys.Num() == InPoints.Num()))
		{
			return;
		}
		
		Super::AddPoints_BeforeLatticeMoved(InPoints, [this, &InKeys](int32 InputIndex, const FPointIndex& PointIndex)
		{
			TArray<TPointMetaData>& MetaData = CellMetaData[PointIndex.CellIndex].PointMetaData;
			MetaData.SetNum(PointIndex.IndexInCell + 1); // Handles the case that API user called Super::AddPoints_BeforeLatticeMoved 
			MetaData[PointIndex.IndexInCell] = InKeys[InputIndex];
		});
	}
	
	/** Moves the specified control points and recomputes the affected points. */
	void UpdateControlPoints(TConstArrayView<int32> InControlPointsToUpdate, TConstArrayView<FVector2D> InNewControlPoints,
		const TFunctionRef<void(const TPointMetaData&, const FVector2D&)> HandleKeyChange
		)
	{
		Super::UpdateControlPoints(InControlPointsToUpdate, InNewControlPoints,
			[this, &HandleKeyChange](const FPointIndex& PointIndex, const FVector2D& UpdatedPosition)
			{
				const TPointMetaData& MetaData = CellMetaData[PointIndex.CellIndex].PointMetaData[PointIndex.IndexInCell];
				HandleKeyChange(MetaData, UpdatedPosition);
			});
	}
	
	/** @return The key IDs of the keys in the cell. */
	TConstArrayView<TPointMetaData> GetCellMetaData(int32 InCellIndex) const
	{
		return ensure(CellMetaData.IsValidIndex(InCellIndex)) ? CellMetaData[InCellIndex].PointMetaData : TConstArrayView<TPointMetaData>();
	}
	
	/** Sets the key IDs of the keys in the cell. */
	void SetCellMetaData(int32 InCellIndex, TArray<TPointMetaData> InNewKeyCoords)
	{
		if (ensure(CellMetaData.IsValidIndex(InCellIndex)))
		{
			CellMetaData[InCellIndex].PointMetaData = MoveTemp(InNewKeyCoords);
		}
	}

private:

	struct FCellMetaData
	{
		/** Same length as the equivalent KeyCoords array of FCellData. */
		TArray<TPointMetaData> PointMetaData;
	};

	/** Same length as FLatticeDeformer2D::Cells */
	TArray<FCellMetaData> CellMetaData;
};
}

