// Copyright Epic Games, Inc. All Rights Reserved.


#include "LatticeDeformerState.h"

#include "Algo/Transform.h"
#include "Misc/LatticeDrawUtils.h"
#include "Misc/VectorMathUtils.h"

namespace UE::CurveEditorTools
{
FLatticeDeformerState::FLatticeDeformerState(const FLatticeBounds& Lattice, const TSharedRef<FCurveEditor>& InCurveEditor)
	: ControlPointToCurveSpace(
		TransformRectBetweenSpaces(Lattice.MinValues, Lattice.MaxValues, Lattice.MinValuesCurveSpace, Lattice.MaxValuesCurveSpace)
		)
	, GlobalDeformer(1, 1, Lattice.MinValues, Lattice.MaxValues)
	, PerCurveData(BuildPerLatticeData(Lattice, *InCurveEditor))
	, CurveChangeListener(CurveEditor::FCurveChangeListener::MakeForAllCurves(InCurveEditor))
	, PanelRebuildListener(InCurveEditor)
{}
	
TArray<FVector2D> FLatticeDeformerState::TransformControlPointsToCurveSpace() const
{
	const TConstArrayView<FVector2D> ControlPoints = GlobalDeformer.GetControlPoints();
	TArray<FVector2D> Result;
	Result.Reserve(ControlPoints.Num());
	Algo::Transform(ControlPoints, Result, [this](const FVector2D& ControlPoint)
	{
		const FVector2D Transformed = ControlPointToCurveSpace.TransformPoint(ControlPoint);
		return Transformed;
	});
	return Result;
}
}
