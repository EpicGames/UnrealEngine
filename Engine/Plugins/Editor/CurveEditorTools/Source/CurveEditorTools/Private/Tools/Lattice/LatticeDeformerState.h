// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CurveEditorTypes.h"
#include "LatticeFwd.h"
#include "Math/TransformCalculus2D.h"
#include "Misc/CurveChangeListener.h"
#include "Misc/CurveViewRebuildListener.h"
#include "Misc/EnumClassFlags.h"
#include "PerCurveLatticeData.h"

namespace UE::CurveEditorTools
{
enum class ELatticeUpdateFlags : uint8
{
	None,
	
	/**
	 * Tries to restore the grid bounds from what is saved in the undo object.
	 * This is set when an undo operation is performed.
	 *
	 * If the data is compatible, the grid is restored to that and the grid is not recomputed.
	 * If the data is incompatible, the entire deformer is recomputed.
	 *
	 * The data may be incompatible, if e.g. you switched from normalized view to absolute view.
	 */
	TryRestoreBoundsFromUndo = 1 << 1,
};
ENUM_CLASS_FLAGS(ELatticeUpdateFlags);

/** Holds any data that exists only when the deformer exists. The deformer exists the selection consists of least 2 keys at different locations. */
struct FLatticeDeformerState
{
	/**
	 * Transforms the absolute Deformer.GetControlPoints(), which are just the FKeyPosition values, to curve space values of the current view.
	 *
	 * Reminder: CurveSpace are axis values that the SCurveEditorView displays on its axis.
	 * In Absolute mode, that's the literal FKeyPosition values
	 * In Normalized mode, Y is in 0 to 1 range (and X continues to be FKeyPosition::InputValue).
	 */
	FTransform2d ControlPointToCurveSpace;

	/**
	 * Empty "deformer" that exists purely for the UI. Its control points, edges, etc. are displayed in the view.
	 * 
	 * Each curve has its own lattice deformer that actually moves that curves keys.
	 * The per curve deformers control points are the result of applying the per-curve transform to Deformer.
	 * This is needed to support all view modes (Absolute, Normalized, etc.)
	 *
	 * The Deformer's control points are always in absolute key space (i.e. exactly the values for FKeyPosition::InputValue and OutputValue).
	 */
	FGlobalLatticeDeformer2D GlobalDeformer;
	TMap<FCurveModelID, FPerCurveLatticeData> PerCurveData; 

	/** Tells us when a curve is externally modified, so we can update the bounds. */
	CurveEditor::FCurveChangeListener CurveChangeListener;
	/** Tells us when the panel is regenerated, in which case we must regenerate the lattice shape (e.g. could go from absolute -> normalized mode). */
	FCurveViewRebuildListener PanelRebuildListener;
	
	/** Guard to not recompute bounds when FCurveModel::OnCurveModified broadcasts due to a change we have initiated. */
	bool bIsModifyingCurves = false;
	/** If true, the curves have been modified and the lattice overlay should be recomputed next frame. */
	bool bHasRequestedRefresh = false;
	/** The flags to use when bHasRequestedRefresh == true. */
	ELatticeUpdateFlags RefreshFlags = ELatticeUpdateFlags::None;

	explicit FLatticeDeformerState(const FLatticeBounds& Lattice, const TSharedRef<FCurveEditor>& InCurveEditor);

	/** @return Control points transformed to SCurveEditorView's curve space. */
	TArray<FVector2D> TransformControlPointsToCurveSpace() const;
};
}
