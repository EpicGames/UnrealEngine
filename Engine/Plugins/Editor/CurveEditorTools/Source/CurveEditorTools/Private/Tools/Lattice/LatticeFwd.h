// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

struct FKeyHandle;

namespace UE::CurveEditorTools
{
class FLatticeDeformer2D;
struct FLatticePointMetaData;
template<typename TPointMetaData> class TLatticeDeformer2D;

/** We never add any points to this deformer. It's just used to keep track of the control points & edges displayed in the view. */
using FGlobalLatticeDeformer2D = FLatticeDeformer2D;
using FPerCurveDeformer2D = TLatticeDeformer2D<FLatticePointMetaData>;
}
