// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveDataAbstraction.h"
#include "Curves/KeyHandle.h"
#include "Serialization/Archive.h"

namespace UE::CurveEditorTools
{
/** Extra that stored for each point added to the lattice grid. */
struct FLatticePointMetaData
{
	/** The handle for the key this lattice point corresponds to. */
	FKeyHandle KeyHandle;
	
	/** The position the key had before the lattice started moving it. */
	FKeyPosition OriginalPosition;
	
	friend FArchive& operator<<(FArchive& InArchive, FLatticePointMetaData& InMetaData);
};

inline FArchive& operator<<(FArchive& InArchive, FLatticePointMetaData& InMetaData)
{
	InArchive << InMetaData.KeyHandle;
	InArchive << InMetaData.OriginalPosition.InputValue;
	InArchive << InMetaData.OriginalPosition.OutputValue;
	return InArchive;
}
}
