// Copyright Epic Games, Inc. All Rights Reserved.

#include "LatticeEdgeTangentMirrorOp.h"

#include "MirrorUtils.h"
#include "Misc/Mirror/TangentMirrorSolver.h"
#include "Misc/Optional.h"
#include "Tools/Lattice/LatticeDeformer2D.h"
#include "Tools/Lattice/LatticePointMetaData.h"
#include "Tools/Lattice/Misc/LatticeUtils.h"

namespace UE::CurveEditorTools
{
namespace MirrorEdgeDetail
{
static bool CanMirror(const FLatticeDeformer2D& InDeformer)
{
	return InDeformer.NumCells() == 1;
}
	
static bool GetDragStartAndMidpointHeights(
	int32 InEdgeIndex, const FLatticeDeformer2D& InDeformer, double& OutDragStartHeight, double& OutMidPointHeight
)
{
	const ELatticeEdgeType OutEdgeType = GetEdgeTypeFromIndexInSingleCellGrid(InEdgeIndex);
	const ELatticeEdgeType OppositeEdgeType = GetOppositeEdge(OutEdgeType);
	const int32 OppositeEdgeIndex = GetEdgeIndexInSingleCellGrid(OppositeEdgeType);

	const auto[StartVertIndex, EndVertIndex] = InDeformer.GetEdgeIndices(InEdgeIndex);
	const auto[OppositeStartVertIndex, OppositeEndVertIndex] = InDeformer.GetEdgeIndices(OppositeEdgeIndex);

	const TConstArrayView<FVector2D> ControlPoints = InDeformer.GetControlPoints();
	OutDragStartHeight = ChooseConsistentEdgeVert(OutEdgeType, ControlPoints[StartVertIndex], ControlPoints[EndVertIndex]);
	OutMidPointHeight = ChooseConsistentEdgeVert(OutEdgeType, ControlPoints[OppositeStartVertIndex], ControlPoints[OppositeEndVertIndex]);
	return true;
}
}

TOptional<FLatticeEdgeTangentsMirrorOp> FLatticeEdgeTangentsMirrorOp::MakeMirrorOpForDragLatticeEdge(
	int32 InEdgeIndex, const FCurveModelID& InCurveId, const FPerCurveDeformer2D& InDeformer, const FCurveEditor& InCurveEditor
	)
{
	if (!MirrorEdgeDetail::CanMirror(InDeformer))
	{
		return {};
	}
	
	TOptional<FCurveMirrorData> MirrorData = ComputeTangentMirrorData(InEdgeIndex, InCurveEditor, InDeformer, InCurveId, InDeformer.GetCellMetaData(0));
	if (!MirrorData)
	{
		return {};
	}
	
	return FLatticeEdgeTangentsMirrorOp(InEdgeIndex, MoveTemp(*MirrorData));
}

void FLatticeEdgeTangentsMirrorOp::OnMoveEdge(TConstArrayView<FVector2D> InNewEdge, const FCurveEditor& InCurveEditor)
{
	if (!ensureMsgf(InNewEdge.Num() == 2, TEXT("We expected an edge")))
	{
		return;
	}
	
	FCurveModel* CurveModel = InCurveEditor.FindCurve(CurveMirrorData.CurveId);
	if (!CurveModel)
	{
		return;
	}
		
	const ELatticeEdgeType EdgeType = GetEdgeTypeFromIndexInSingleCellGrid(EdgeIndex);
	CurveMirrorData.Solver.OnMoveEdge(InCurveEditor, ChooseConsistentEdgeVert(EdgeType, InNewEdge[0], InNewEdge[1]));
}

TOptional<FLatticeEdgeTangentsMirrorOp::FCurveMirrorData> FLatticeEdgeTangentsMirrorOp::ComputeTangentMirrorData(
	int32 InEdgeIndex,
	const FCurveEditor& InCurveEditor, const FLatticeDeformer2D& InCurveDeformer, const FCurveModelID& InCurve,
	TConstArrayView<FLatticePointMetaData> InMetaData
	)
{
	double DragStartHeight, MidPointHeight;
	if (!MirrorEdgeDetail::GetDragStartAndMidpointHeights(InEdgeIndex, InCurveDeformer, DragStartHeight, MidPointHeight))
	{
		return {};
	}
	
	TArray<FKeyHandle> KeyHandles;
	Algo::Transform(InMetaData, KeyHandles, [](const FLatticePointMetaData& MetaData) { return MetaData.KeyHandle;} );
	
	CurveEditor::FTangentMirrorSolver TangentSolver(DragStartHeight, MidPointHeight);
	const bool bSuccess = TangentSolver.AddTangents(InCurveEditor, InCurve, KeyHandles);
	return bSuccess ? FCurveMirrorData(InCurve, MoveTemp(TangentSolver)) : TOptional<FCurveMirrorData>();
}
}
