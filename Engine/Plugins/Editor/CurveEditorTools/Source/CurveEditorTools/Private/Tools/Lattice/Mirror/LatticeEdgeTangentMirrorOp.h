// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MirrorUtils.h"
#include "Misc/Mirror/TangentMirrorSolver.h"
#include "Tools/Lattice/LatticeFwd.h"

class FCurveEditor;
namespace UE::CurveEditor { class FTangentMirrorSolver; }
template<typename OptionalType> struct TOptional;

/** Implements mirroring functions based off of lattice tool's data. */
namespace UE::CurveEditorTools
{
class FLatticeDeformer2D;
	
/**
 * Provides functionality to drag the top or bottom edge of a lattice grid and mirror the tangents accordingly.
 * @note This operation only affects the tangents. Key positions must be adjusted separately using the lattice grid algorithm.
 */
class FLatticeEdgeTangentsMirrorOp
{
public:

	/**
	 * Util function that creates a mirror op for dragging InEdgeIndex.
	 * @return Valid if InEdgeIndex is an edge that should do mirroring when dragged, i.e. a top or bottom edge
	 */
	static TOptional<FLatticeEdgeTangentsMirrorOp> MakeMirrorOpForDragLatticeEdge(
		int32 InEdgeIndex, const FCurveModelID& InCurveId, const FPerCurveDeformer2D& InDeformer,
		const FCurveEditor& InCurveEditor UE_LIFETIMEBOUND
	);
	
	/** Call with info about where the new edge is located. */
	void OnMoveEdge(TConstArrayView<FVector2D> InNewEdge, const FCurveEditor& InCurveEditor);
	
private:

	/** Indexs of the edge being moved. */
	const int32 EdgeIndex;

	struct FCurveMirrorData
	{
		FCurveModelID CurveId;
		CurveEditor::FTangentMirrorSolver Solver;

		explicit FCurveMirrorData(FCurveModelID InCurveId, CurveEditor::FTangentMirrorSolver&& InSolver)
			: CurveId(InCurveId), Solver(MoveTemp(InSolver))
		{}
	};
	/** Computes the tangents */
	FCurveMirrorData CurveMirrorData;
	
	explicit FLatticeEdgeTangentsMirrorOp(int32 InEdgeIndex, FCurveMirrorData InMirrorData)
		: EdgeIndex(InEdgeIndex), CurveMirrorData(MoveTemp(InMirrorData))
	{}
	
	/** Tracks keys that have tangents that can be mirrored. The mirror midpoints are computed based off of InCurveDeformer. */
	static TOptional<FCurveMirrorData> ComputeTangentMirrorData(
		int32 InEdgeIndex, const FCurveEditor& InCurveEditor,
		const FLatticeDeformer2D& InCurveDeformer, const FCurveModelID& InCurve, TConstArrayView<FLatticePointMetaData> InMetaData
		);
};
}
