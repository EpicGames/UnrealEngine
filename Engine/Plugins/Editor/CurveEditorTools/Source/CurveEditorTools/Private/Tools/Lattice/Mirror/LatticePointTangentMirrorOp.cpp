// Copyright Epic Games, Inc. All Rights Reserved.

#include "LatticePointTangentMirrorOp.h"

#include "Algo/AllOf.h"
#include "MirrorUtils.h"
#include "Misc/Optional.h"
#include "Tools/Lattice/LatticeDeformer2D.h"
#include "Tools/Lattice/LatticePointMetaData.h"
#include "Tools/Lattice/Misc/LatticeUtils.h"

namespace UE::CurveEditorTools
{
namespace MirrorPointDetail
{
enum class EEdgeSide { Left, Right };
	
static void GetDragStartAndMidpointHeights(
	int32 InControlPointIndex, const FLatticeDeformer2D& InDeformer, EEdgeSide& OutEdgeType, double& OutDragStartHeight, double& OutMidPointHeight
)
{
	constexpr int32 TopIndex = GetEdgeIndexInSingleCellGrid(ELatticeEdgeType::Top);
	constexpr int32 BottomIndex = GetEdgeIndexInSingleCellGrid(ELatticeEdgeType::Bottom);
	const auto[TopStartVertIndex, TopEndVertIndex] = InDeformer.GetEdgeIndices(TopIndex);
	const auto[BottomStartVertIndex, BottomEndVertIndex] = InDeformer.GetEdgeIndices(BottomIndex);

	const auto[TopLeft, TopRight, BottomRight, BottomLeft] = InDeformer.GetCellIndices(0);
	const bool bIsLeft = InControlPointIndex == TopLeft || InControlPointIndex == BottomLeft;
	OutEdgeType = bIsLeft ? EEdgeSide::Left : EEdgeSide::Right;

	const TConstArrayView<FVector2D> ControlPoints = InDeformer.GetControlPoints();
	const bool bDraggedIsTopEdge = InControlPointIndex == TopStartVertIndex || InControlPointIndex == TopEndVertIndex;
	OutDragStartHeight = InDeformer.GetControlPoints()[InControlPointIndex].Y;
	OutMidPointHeight = bDraggedIsTopEdge
		? ChooseConsistentEdgeVert(ELatticeEdgeType::Bottom, ControlPoints[BottomStartVertIndex], ControlPoints[BottomEndVertIndex])
		: ChooseConsistentEdgeVert(ELatticeEdgeType::Top, ControlPoints[TopStartVertIndex], ControlPoints[TopEndVertIndex]);
}
}

TOptional<FLatticePointTangentsMirrorOp> FLatticePointTangentsMirrorOp::MakeMirrorOpForDragLatticeControlPoint(
	int32 InControlPointIndex, const FCurveModelID& InCurveId, const FPerCurveDeformer2D& InDeformer, const FCurveEditor& InCurveEditor)
{
	TOptional<FCurveMirrorData> CurveData = ComputeTangentMirrorData(InControlPointIndex, InCurveEditor, InCurveId, InDeformer);
	if (CurveData)
	{
		return FLatticePointTangentsMirrorOp(MoveTemp(*CurveData));
	}
	return {};
}

void FLatticePointTangentsMirrorOp::OnMovePoint(const FVector2D& InControlPoint, const FCurveEditor& CurveEditor)
{
	const FCurveModelID& CurveId = CurveData.CurveId;
		
	FCurveModel* CurveModel = CurveEditor.FindCurve(CurveId);
	if (!CurveModel)
	{
		return;
	}
		
	CurveEditor::FCurveTangentMirrorData& MirrorData = CurveData.TangentMirrorSolver.CurveData[CurveId];
	RecomputeMirroringParallel(CurveEditor, CurveId, MirrorData, InControlPoint.Y,
		// Flatten the tangents more if their key is closer to the side of the dragged control point, and don't flatten at all on opposite side.
		// Effectively, this interpolates the interpolated tangents...
		[this, &MirrorData](int32 KeyIndex, const FVector2D& InterpolatedTangents)
		{
			const float Alpha = CurveData.FalloffValues[KeyIndex];
			const float OneMinus = 1.f - Alpha;
			// Weighted average of interpolated and original tangent value. Effectively, this is a linear falloff.
			return Alpha * InterpolatedTangents + OneMinus * MirrorData.TangentSolver.InitialValues[KeyIndex];
		});
}

TOptional<FLatticePointTangentsMirrorOp::FCurveMirrorData> FLatticePointTangentsMirrorOp::ComputeTangentMirrorData(
	int32 InControlPointIndex, const FCurveEditor& InCurveEditor, const FCurveModelID& InCurveId, const FPerCurveDeformer2D& InDeformer
	)
{
	using namespace MirrorPointDetail;
	if (!ensure(InDeformer.NumCells() == 1))
	{
		return {};
	}
	
	EEdgeSide EdgeSide;
	double DragStartHeight, MidPointHeight;
	GetDragStartAndMidpointHeights(InControlPointIndex, InDeformer, EdgeSide, DragStartHeight, MidPointHeight);
		
	const auto[TopLeftIdx, TopRightIdx, BottomRightIdx, BottomLeftIdx] = InDeformer.GetCellIndices(0);
	const TConstArrayView<FVector2D> ControlPoints = InDeformer.GetControlPoints();
	const double MinX = FMath::Min(ControlPoints[TopLeftIdx].X, ControlPoints[BottomLeftIdx].X);
	const double MaxX = FMath::Min(ControlPoints[TopRightIdx].X, ControlPoints[BottomRightIdx].X);
	const double Width = MaxX - MinX;
	TArray<float> FalloffValues;
	
	// We're going to compute falloff values for each key: between 0 and 1.
	// The closer the key is to the other side of the lattice grid, the closer the falloff value is to 0.
	// The closer the key is to the control point, the closer the falloff value is to 1. 
	const auto ComputeFalloff = [&InCurveEditor, EdgeSide, MinX, Width, &FalloffValues, &InCurveId]
		(const CurveEditor::FMirrorableTangentInfo& InTangentInfo)
	{
		const FCurveModel* Curve = InCurveEditor.FindCurve(InCurveId);
		const int32 NumKeys = InTangentInfo.MirrorableKeys.Num();
		TArray<FKeyPosition> Positions; 
		Positions.SetNumUninitialized(NumKeys, EAllowShrinking::No);
		Curve->GetKeyPositions(InTangentInfo.MirrorableKeys, Positions);

		FalloffValues.SetNumUninitialized(NumKeys);
		for (int32 Index = 0; Index < NumKeys; ++Index)
		{
			// If key is on the left = 0, and on right = 1.
			const float PercentFromLeft = (Positions[Index].InputValue - MinX) / Width;
			// If control point is on the left		-> key on left side -> falloff = 1,		key on right side -> falloff = 0
			// If control point is on the right		-> key on left side -> falloff = 0,		key on right side -> falloff = 1
			const float Falloff = EdgeSide == EEdgeSide::Left ? 1.f - PercentFromLeft : PercentFromLeft;
			FalloffValues[Index] = Falloff;
		}
	};
	
	TArray<FKeyHandle> KeyHandles;
	const TConstArrayView<FLatticePointMetaData> CellMetaData = InDeformer.GetCellMetaData(0);
	Algo::Transform(CellMetaData, KeyHandles, [](const FLatticePointMetaData& MetaData) { return MetaData.KeyHandle;} );
	
	CurveEditor::FTangentMirrorSolver MirrorSolver(DragStartHeight, MidPointHeight);
	const bool bHadTangentsToMirror = MirrorSolver.AddTangents(InCurveEditor, InCurveId, KeyHandles, ComputeFalloff);
	// Only user-specified tangents need mirroring
	return bHadTangentsToMirror ? FCurveMirrorData(InCurveId, MoveTemp(MirrorSolver), MoveTemp(FalloffValues)) : TOptional<FCurveMirrorData>();
}
}
