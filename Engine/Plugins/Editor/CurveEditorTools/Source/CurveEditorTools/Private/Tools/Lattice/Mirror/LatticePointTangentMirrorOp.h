// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/Mirror/TangentMirrorSolver.h"
#include "Tools/Lattice/LatticeDeformer2D.h"
#include "Tools/Lattice/LatticeFwd.h"

class FCurveEditor;
namespace UE::CurveEditor { class FTangentMirrorSolver; }
template<typename OptionalType> struct TOptional;

/** Implements mirroring functions based off of lattice tool's data. */
namespace UE::CurveEditorTools
{
class FLatticeDeformer2D;
struct FKeyInCurveId;

/**
 * Provides functionality to drag a control point of a lattice grid and mirror the tangents accordingly.
 *
 * This implementation differs from dragging an edge: a falloff is applied to all tangents. Tangents are flattened the full amount the closer they are
 * to the dragged control point. Tangents of keys on the other side of the lattice grid are flattened less.
 * 
 * @note This operation only affects the tangents. Key positions must be adjusted separately using the lattice grid algorithm.
 */
class FLatticePointTangentsMirrorOp
{
public:

	/**
	 * When dragging InControlPointIndex, checks whether the deformer contains keys that need mirroring, which are those that are user specified.
	 * @return Set if there were keys to mirror when dragging the specifiec control point. Unset otherwise.
	 */
	static TOptional<FLatticePointTangentsMirrorOp> MakeMirrorOpForDragLatticeControlPoint(
		int32 InControlPointIndex, const FCurveModelID& InCurveId, const FPerCurveDeformer2D& InDeformer,
		const FCurveEditor& InCurveEditor UE_LIFETIMEBOUND
		);
	
	/** Call with info about where the new control point is located. */
	void OnMovePoint(const FVector2D& InControlPoint, const FCurveEditor& CurveEditor);

private:
	
	struct FCurveMirrorData
	{
		/** CurveId of the curve whose tangents are being mirrored */
		FCurveModelID CurveId;
		/** Mirrors the curve's keys. */
		CurveEditor::FTangentMirrorSolver TangentMirrorSolver;
		/** Equal length as TangentMirrorSolver.CurveData.KeyHandles. Contains the falloff values we pre-computed for each key. */
		TArray<float> FalloffValues;

		explicit FCurveMirrorData(FCurveModelID CurveId, CurveEditor::FTangentMirrorSolver&& TangentMirrorSolver, TArray<float>&& FalloffValues)
			: CurveId(CurveId), TangentMirrorSolver(MoveTemp(TangentMirrorSolver)), FalloffValues(MoveTemp(FalloffValues))
		{}
	};
	/** Per curve data for performing mirroring. */
	FCurveMirrorData CurveData;
	
	explicit FLatticePointTangentsMirrorOp(FCurveMirrorData&& InCurveData)
		: CurveData(MoveTemp(InCurveData))
	{}
	
	/**
	 * Adds all keys that can be mirrored to this operation.
	 * @return True, if any of the lattice's keys need mirroring.
	 */
	static TOptional<FCurveMirrorData> ComputeTangentMirrorData(
		int32 InControlPointIndex, const FCurveEditor& CurveEditor, const FCurveModelID& InCurveId, const FPerCurveDeformer2D& InDeformer
		);
};
}
