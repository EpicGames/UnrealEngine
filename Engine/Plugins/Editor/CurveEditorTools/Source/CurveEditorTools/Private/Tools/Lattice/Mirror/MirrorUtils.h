// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector.h"
#include "Misc/AssertionMacros.h"
#include "Tools/Lattice/Misc/LatticeUtils.h"

namespace UE::CurveEditorTools
{
/** Consistenly chooses the same vertex on the top or bottom edge ASSUMING that the vertices are always transformed uniformly (e.g. move both up by x units, etc.) */
inline double ChooseConsistentEdgeVert(ELatticeEdgeType EdgeType, const FVector2D& Vert1, const FVector2D& Vert2)
{
	// In several places, we need to consistently choose vert 1 or vert 2 to take the height from.
	// During the edge drag operation the vertices are moved up and down by the same amounts.
	// For top edge we choose the higher, and for bottom edge the lower vertex on the edge, which guarantees we consistently choose the same vertex.
	// Choosing top and bottom vertex, respectively, is done so with non-parallel edges the tangents are interpolated as slowly as possible, i.e.
	// you need to move the mouse more. 
	switch (EdgeType)
	{
	case ELatticeEdgeType::Top: return FMath::Max(Vert1.Y, Vert2.Y);
	case ELatticeEdgeType::Bottom: return FMath::Min(Vert1.Y, Vert2.Y);
	case ELatticeEdgeType::Left: return FMath::Max(Vert1.X, Vert2.X);
	case ELatticeEdgeType::Right: return FMath::Min(Vert1.X, Vert2.X);
		
	default: checkNoEntry(); return 0;
	}
}
}
