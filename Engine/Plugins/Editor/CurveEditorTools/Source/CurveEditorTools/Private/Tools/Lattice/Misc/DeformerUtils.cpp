// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeformerUtils.h"

#include "Misc/Optional.h"

namespace UE::CurveEditorTools
{
TOptional<FMoveEdgeToOppositeData> ComputeMoveEdgeToOppositeData(const FLatticeDeformer2D& InDeformer, int32 InEdgeIndex)
{
	// At time of implementation, we only have a single cell. Should that change, adjust this implementation.
	if (InDeformer.NumCells() != 1)
	{
		return {};
	}

	const ELatticeEdgeType EdgeType = GetEdgeTypeFromIndexInSingleCellGrid(InEdgeIndex);
	if (EdgeType != ELatticeEdgeType::Top && EdgeType != ELatticeEdgeType::Bottom)
	{
		return {};
	}

	const ELatticeEdgeType OppositeEdge = GetOppositeEdge(EdgeType);
	const int32 OppositeEdgeIndex = GetEdgeIndexInSingleCellGrid(OppositeEdge);
	const FEdgeVertexIndices Indices = InDeformer.GetEdgeIndices(OppositeEdgeIndex);
	const FLatticeControlEdge& Edge = InDeformer.GetControlEdge(InEdgeIndex);
	const TArray<FVector2D, TInlineAllocator<2>> NewEdge = { Edge.Start(), Edge.End() };
	return FMoveEdgeToOppositeData{ OppositeEdgeIndex, Indices, Edge };
}
}

