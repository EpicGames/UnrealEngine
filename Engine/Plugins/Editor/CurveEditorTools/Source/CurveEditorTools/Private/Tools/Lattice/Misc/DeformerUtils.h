// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "LatticeUtils.h"
#include "Tools/Lattice/LatticeDeformer2D.h"

template<typename OptionalType> struct TOptional;

namespace UE::CurveEditorTools
{
struct FMoveEdgeToOppositeData
{
	int32 OppositeEdgeIndex;
	FEdgeVertexIndices EdgeIndices;
	FLatticeControlEdge EdgeControlPoints;
};

/** @return Information about the edge to move to the opposite. Can only move the top or bottom edge. */
TOptional<FMoveEdgeToOppositeData> ComputeMoveEdgeToOppositeData(const FLatticeDeformer2D& InDeformer, int32 InEdgeIndex);
}
