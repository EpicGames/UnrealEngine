// Copyright Epic Games, Inc. All Rights Reserved.

#include "LatticeDrawUtils.h"

#include "CurveEditor.h"
#include "CurveEditorSelection.h"
#include "CurveModel.h"
#include "LatticeUtils.h"
#include "Math/NumericLimits.h"
#include "Misc/VectorMathUtils.h"
#include "Rendering/DrawElements.h"
#include "SCurveEditorPanel.h"
#include "SCurveEditorView.h"
#include "Tools/Lattice/LatticeControlsDrawData.h"
#include "Tools/Lattice/LatticeDeformer2D.h"

namespace UE::CurveEditorTools
{	
FSlateLayoutTransform ComputeViewToViewContainerTransform(const SCurveEditorView& InView, const FCurveEditor& InCurveEditor)
{
	const FSlateLayoutTransform ViewToAbsolute = InView.GetCachedGeometry().GetAccumulatedLayoutTransform();
	const FSlateLayoutTransform ViewContainerToAbsolute = InCurveEditor.GetPanel()->GetViewContainerGeometry().GetAccumulatedLayoutTransform();
	const FSlateLayoutTransform AbsoluteToViewContainer = ViewContainerToAbsolute.Inverse();
	const FSlateLayoutTransform ViewToViewContainerTransform = Concatenate(ViewToAbsolute, AbsoluteToViewContainer);
	return ViewToViewContainerTransform;
}

FLatticeBounds FLatticeBounds::MakeCombined(const FLatticeBounds& InFirst, const FLatticeBounds& InOther)
{
	return FLatticeBounds
	{
		InFirst.bIsVisible || InOther.bIsVisible,
		FVector2d::Min(InFirst.MinValues, InOther.MinValues),
		FVector2d::Max(InFirst.MaxValues, InOther.MaxValues),
		FVector2d::Min(InFirst.MinSlatePosition, InOther.MinSlatePosition),
		FVector2d::Max(InFirst.MaxSlatePosition, InOther.MaxSlatePosition),
		FVector2d::Min(InFirst.MinValuesCurveSpace, InOther.MinValuesCurveSpace),
		FVector2d::Max(InFirst.MaxValuesCurveSpace, InOther.MaxValuesCurveSpace)
	};
}
	
FLatticeBounds ComputeBounds(const FCurveEditor& InCurveEditor)
{
	FLatticeBounds Bounds;
	Bounds.MinValues = { TNumericLimits<double>::Max(), TNumericLimits<double>::Max() };
	Bounds.MaxValues = { TNumericLimits<double>::Lowest(), TNumericLimits<double>::Lowest() };
	Bounds.MinSlatePosition = { TNumericLimits<double>::Max(), TNumericLimits<double>::Max() };
	Bounds.MaxSlatePosition = { TNumericLimits<double>::Lowest(), TNumericLimits<double>::Lowest() };
	Bounds.MinValuesCurveSpace = { TNumericLimits<double>::Max(), TNumericLimits<double>::Max() };
	Bounds.MaxValuesCurveSpace = { TNumericLimits<double>::Lowest(), TNumericLimits<double>::Lowest() };
	
	const TMap<FCurveModelID, FKeyHandleSet>& SelectedKeySet = InCurveEditor.Selection.GetAll();
	for (const TPair<FCurveModelID, FKeyHandleSet>& Pair : SelectedKeySet)
	{
		Bounds = FLatticeBounds::MakeCombined(Bounds, ComputeCurveBounds(InCurveEditor, Pair.Key, Pair.Value.AsArray()));
	}
	
	return Bounds;
}
	
FLatticeBounds ComputeCurveBounds(const FCurveEditor& InCurveEditor, const FCurveModelID& InCurveId, TConstArrayView<FKeyHandle> InKeys)
{
	FLatticeBounds Bounds;
	Bounds.MinValues = { TNumericLimits<double>::Max(), TNumericLimits<double>::Max() };
	Bounds.MaxValues = { TNumericLimits<double>::Lowest(), TNumericLimits<double>::Lowest() };
	Bounds.MinSlatePosition = { TNumericLimits<double>::Max(), TNumericLimits<double>::Max() };
	Bounds.MaxSlatePosition = { TNumericLimits<double>::Lowest(), TNumericLimits<double>::Lowest() };
	Bounds.MinValuesCurveSpace = { TNumericLimits<double>::Max(), TNumericLimits<double>::Max() };
	Bounds.MaxValuesCurveSpace = { TNumericLimits<double>::Lowest(), TNumericLimits<double>::Lowest() };

	const SCurveEditorView* View = InCurveEditor.FindFirstInteractiveView(InCurveId);
	if (!View)
	{
		return Bounds;
	}

	// A newly created view may have a zero-size until the next tick which is a problem if
	// we ask the View for it's curve space, so we skip over it until it has a size.
	if(View->GetCachedGeometry().GetLocalSize() == FVector2D::ZeroVector)
	{
		return Bounds;
	}
	
	FCurveModel* CurveModel = InCurveEditor.FindCurve(InCurveId);
	check(CurveModel);

	TArray<FKeyPosition> KeyPositions;
	KeyPositions.SetNumUninitialized(InKeys.Num());
	CurveModel->GetKeyPositions(InKeys, KeyPositions);
	
	const FTransform2d AbsToCurveSpace = View->GetViewToCurveTransform(InCurveId);
	double M00, M01, M10, M11;
	AbsToCurveSpace.GetMatrix().GetMatrix(M00, M01, M10, M11);

	FCurveEditorScreenSpace ViewSpace = View->GetCurveSpace(InCurveId);
	const FSlateLayoutTransform ViewToViewContainerTransform = ComputeViewToViewContainerTransform(*View, InCurveEditor);
	for (int32 i = 0; i < KeyPositions.Num(); ++i)
	{
		const FKeyPosition& KeyPosition = KeyPositions[i];
		const FVector2D Position_ViewSpace = FVector2D(
			ViewSpace.SecondsToScreen(KeyPosition.InputValue), ViewSpace.ValueToScreen(KeyPosition.OutputValue)
			);
		const FVector2D PanelSpaceLocation = ViewToViewContainerTransform.TransformPoint(Position_ViewSpace);

		Bounds.bIsVisible = true;
		
		Bounds.MinValues.X = FMath::Min(KeyPosition.InputValue, Bounds.MinValues.X);
		Bounds.MinValues.Y = FMath::Min(KeyPosition.OutputValue, Bounds.MinValues.Y);
		Bounds.MaxValues.X = FMath::Max(KeyPosition.InputValue, Bounds.MaxValues.X);
		Bounds.MaxValues.Y = FMath::Max(KeyPosition.OutputValue, Bounds.MaxValues.Y);
		
		Bounds.MinSlatePosition = FVector2D::Min(PanelSpaceLocation, Bounds.MinSlatePosition);
		Bounds.MaxSlatePosition = FVector2D::Max(PanelSpaceLocation, Bounds.MaxSlatePosition);

		const FVector2D CurveSpacePosition = TransformAbsoluteToCurveSpace(AbsToCurveSpace, FVector2D{ KeyPosition.InputValue, KeyPosition.OutputValue });
		Bounds.MinValuesCurveSpace = FVector2D::Min(CurveSpacePosition, Bounds.MinValuesCurveSpace);
		Bounds.MaxValuesCurveSpace = FVector2D::Max(CurveSpacePosition, Bounds.MaxValuesCurveSpace);
	}

	return Bounds;
}

bool TransformViewToViewContainer(
	const FCurveEditor& InCurveEditor,
	TConstArrayView<FVector2D> InControlPoints,
	TArrayView<FVector2D> OutTransformedToViewContainer
	)
{
	check(InControlPoints.Num() <= OutTransformedToViewContainer.Num());
	
	const SCurveEditorView* View = [&InCurveEditor]() -> const SCurveEditorView*
	{
		for (const TPair<FCurveModelID, FKeyHandleSet>& Pair : InCurveEditor.Selection.GetAll())
		{
			if (const SCurveEditorView* View = InCurveEditor.FindFirstInteractiveView(Pair.Key))
			{
				return View;
			}
		}
		return nullptr;
	}();
	// A newly created view may have a zero-size until the next tick which is a problem if
	// we ask the View for it's curve space, so we skip over it until it has a size.
	if (!View || View->GetCachedGeometry().GetLocalSize() == FVector2D::ZeroVector)
	{
		return false;
	}
	
	// The widget hierarchy is SCurveEditorViewContainer -> SCurveEditorView.
	// ControlPoints are in ViewSpace(SCurveEditorView) and transformed to SCurveEditorViewContainer's FGeometry, received in OnPaint.
	const FCurveEditorScreenSpace ViewSpace = View->GetViewSpace();
	const FSlateLayoutTransform ViewToViewContainerTransform = ComputeViewToViewContainerTransform(*View, InCurveEditor);

	for (int32 Index = 0; Index < InControlPoints.Num(); ++Index)
	{
		const FVector2D& ControlPoint = InControlPoints[Index];
		const FVector2D Position_ViewSpace = FVector2D(
			ViewSpace.SecondsToScreen(ControlPoint.X), ViewSpace.ValueToScreen(ControlPoint.Y)
			);
		const FVector2D Position_ViewContainerSpace = ViewToViewContainerTransform.TransformPoint(Position_ViewSpace);
		OutTransformedToViewContainer[Index] = Position_ViewContainerSpace;
	}
	return true;
}

namespace LatticeDrawConstants
{
constexpr float ControlPointAnchorWidth = 13.f;
constexpr float ControlPointHighlightAlpha = 0.15f;
constexpr float ControlEdgeDashLength = 3.f;
	
constexpr float EdgeHoveredSideSize = 10.f;
constexpr float EdgeHighlightAlpha = 0.15f;
	
constexpr float CellHoverInset = EdgeHoveredSideSize / 2.f;

static FGeometry GetPointGeometry(const FGeometry& InLatticeGeometry, const FVector2D& ControlPoint)
{
	const FVector2D PointBoxSize(ControlPointAnchorWidth, ControlPointAnchorWidth);
	const FVector2D Translation = ControlPoint - FVector2D(ControlPointAnchorWidth / 2.f);
	const FGeometry PointGeometry = InLatticeGeometry.MakeChild(PointBoxSize, FSlateLayoutTransform(Translation));
	return PointGeometry;
}

struct FRotatedEdgeInfo
{
	/** Geometry of the hovered edge rect (unrotated). */
	FGeometry EdgeRectGeometry;
	/** How much to turn EdgeRectGeometry (clockwise - just plug it into Slate's rotation transform as is). */
	float AngleRadians;
	FVector2D Offset;
};

static FRotatedEdgeInfo GetEdgeRenderInfo(const FGeometry& InViewContainerGeometry, const FVector2D& Start, const FVector2D& End, bool bOffsetEdgeCenter = true)
{
	const FVector2D StartToEnd = End - Start;
	const float EdgeLength = StartToEnd.Length();
	const FVector2D EdgeBoxSize(EdgeLength, LatticeDrawConstants::EdgeHoveredSideSize);
	
	// Compute angle between positive X-Axis: A*B=|A||B|cos(a). A = StartToEnd, B = X-Axis, |B|=1. A*B = A.X (as B.Y = 0).
	// Reminder: pos. X -> Angle in [0,90], neg. X, angle in [90,180].
	float Angle = FMath::Acos(StartToEnd.X / EdgeLength);
	// Reminder: Slate's pos. Y-Axis points down, and pos. x-axis to the right. 
	const bool bEdgePointsUp = StartToEnd.Y < 0.f;
	// Mirror ACos if the point is on the neg. side of the y-axis.
	Angle *= bEdgePointsUp ? -1.f : 1.f;
		
	FVector2D Translation = Start;
	// Offset the box's side to be in the center of the control point.
	FVector2D Offset(0.0, LatticeDrawConstants::EdgeHoveredSideSize / 2.0);
	Offset = Offset.GetRotated(FMath::RadiansToDegrees(Angle));
	if (bOffsetEdgeCenter)
	{
		Translation -= Offset;
	}
		
	const FGeometry ChildGeometry = InViewContainerGeometry.MakeChild(EdgeBoxSize, FSlateLayoutTransform(Translation));
	return { ChildGeometry, Angle, Offset};
}
}
	
FLatticeHoverState ComputeLatticeHoverState(
	const FGeometry& InViewContainerGeometry,
	const FVector2D& InMouseScreenPosition,
	TConstArrayView<FVector2D> InControlPoints,
	TConstArrayView<FLatticeControlEdge> InControlEdges,
	int32 InNumPointsInX,
	int32 InNumCells
	)
{
	FLatticeHoverState HoverState;
	
	for (int32 Index = 0; Index < InControlPoints.Num(); ++Index)
	{
		const FVector2D& ControlPoint = InControlPoints[Index];
		const FGeometry PointGeometry = LatticeDrawConstants::GetPointGeometry(InViewContainerGeometry, ControlPoint);
		if (PointGeometry.IsUnderLocation(InMouseScreenPosition))
		{
			HoverState.HoveredControlPoint = Index;
			// Don't hover anything else.
			return HoverState;
		}
	}

	const FSlateRenderTransform& ViewContainerToAbsolute = InViewContainerGeometry.GetAccumulatedRenderTransform();
	const FSlateRenderTransform& AbsoluteToViewContainer = ViewContainerToAbsolute.Inverse();
	const FVector2D Mouse_ViewContainer = AbsoluteToViewContainer.TransformPoint(InMouseScreenPosition);
	for (int32 Index = 0; Index < InControlEdges.Num(); ++Index)
	{
		const FLatticeControlEdge& Edge = InControlEdges[Index];
		const FVector2D& Start = Edge.Start();
		const FVector2D& End = Edge.End();
		constexpr bool bOffsetEdgeCenter = false; // We need to do the offsetting ourselves below.
		const auto[EdgeGeometry, Angle, Offset] = LatticeDrawConstants::GetEdgeRenderInfo(InViewContainerGeometry, Start, End, bOffsetEdgeCenter);

		// We're going to compute the direction vector from start to mouse, rotate it instead of the box, and check whether the adjust mouse position is in the box.
		const FVector2D StartCenter_ViewContainer = Start - Offset;
		const FVector2D StartCenterToMouse_ViewContainer = Mouse_ViewContainer - StartCenter_ViewContainer;
		const FVector2D RotatedStartToMouse_ViewContainer = StartCenterToMouse_ViewContainer.GetRotated(FMath::RadiansToDegrees(-Angle));
		const FVector2D RotatedMouse_ViewContainer = Start + RotatedStartToMouse_ViewContainer;
		const FSlateRect BoundingBox_ScreenSpace = EdgeGeometry.GetLayoutBoundingRect();
		const FVector2D RotatedMousePosition_ScreenSpace = ViewContainerToAbsolute.TransformPoint(RotatedMouse_ViewContainer);
		if (BoundingBox_ScreenSpace.ContainsPoint(RotatedMousePosition_ScreenSpace))
		{
			HoverState.HoveredEdge = Index;
			return HoverState;
		}
	}

	for (int32 CellIndex = 0; CellIndex < InNumCells; ++CellIndex)
	{
		const auto[TopLeft, TopRight, BottomRight, BottomLeft] = GetCellIndices(CellIndex, InNumPointsInX);
		const bool bIsInTriangle1 = IsPointInTriangle(
			Mouse_ViewContainer, InControlPoints[TopLeft], InControlPoints[TopRight], InControlPoints[BottomRight]
			);
		const bool bIsInTriangle2 = IsPointInTriangle(
			Mouse_ViewContainer, InControlPoints[BottomRight], InControlPoints[BottomLeft], InControlPoints[TopLeft]
			);
		const bool bIsInCell = bIsInTriangle1 || bIsInTriangle2;
		if (bIsInCell)
		{
			HoverState.HoveredCell = CellIndex;
			return HoverState;
		}
	}

	return HoverState;
}

static void DrawControlPoints(
	const FLatticeControlsDrawData& InDrawData, const FGeometry& InViewContainerGeometry, FSlateWindowElementList& OutDrawElements, int32 InPaintOnLayerId
	)
{
	for (int32 Index = 0; Index < InDrawData.ControlPoints.Num(); ++Index)
	{
		const FVector2D& ControlPoint = InDrawData.ControlPoints[Index];
		const FGeometry PointGeometry = LatticeDrawConstants::GetPointGeometry(InViewContainerGeometry, ControlPoint);
		FSlateDrawElement::MakeBox(
			OutDrawElements, InPaintOnLayerId, PointGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("MarqueeSelection"))
			);
		
		if (InDrawData.HoverState.HoveredControlPoint.Get(INDEX_NONE) == Index)
		{
			const FLinearColor HighlightColor = FLinearColor::White.CopyWithNewOpacity(LatticeDrawConstants::ControlPointHighlightAlpha);
			FSlateDrawElement::MakeBox(
				OutDrawElements, InPaintOnLayerId, PointGeometry.ToPaintGeometry(), FAppStyle::GetBrush(TEXT("WhiteBrush")),
				ESlateDrawEffect::None, HighlightColor
			);
		}
	}
}

static void DrawControlEdges(
	const FLatticeControlsDrawData& InDrawData, const FGeometry& InViewContainerGeometry, FSlateWindowElementList& OutDrawElements, int32 InPaintOnLayerId
	)
{
	for (int32 Index = 0; Index < InDrawData.ControlEdges.Num(); ++Index)
	{
		const FLatticeControlEdge& Edge = InDrawData.ControlEdges[Index];
		const FVector2D& Start = Edge.Start();
		const FVector2D& End = Edge.End();
		
		FSlateDrawElement::MakeDashedLines(
			OutDrawElements,
			InPaintOnLayerId,
			InViewContainerGeometry.ToPaintGeometry(),
			{ FVector2f(Start.X, Start.Y), FVector2f(End.X, End.Y) },
			ESlateDrawEffect::None,
			FLinearColor::White,
			1.f,
			LatticeDrawConstants::ControlEdgeDashLength
			);

		if (InDrawData.HoverState.HoveredEdge == Index)
		{
			const auto[EdgeGeometry, Angle, _] = LatticeDrawConstants::GetEdgeRenderInfo(InViewContainerGeometry, Start, End);
			FSlateDrawElement::MakeRotatedBox(
				OutDrawElements,
				InPaintOnLayerId,
				EdgeGeometry.ToPaintGeometry(),
				FAppStyle::GetBrush(TEXT("WhiteBrush")),
				ESlateDrawEffect::None,
				Angle,
				FVector2f(0, 0),
				FSlateDrawElement::RelativeToElement,
				FLinearColor::White.CopyWithNewOpacity(LatticeDrawConstants::EdgeHighlightAlpha)
				);
		}
	}
}

static void DrawHoveredCells(
	const FLatticeControlsDrawData& InDrawData, const FGeometry& InViewContainerGeometry, FSlateWindowElementList& OutDrawElements, int32 InPaintOnLayerId
	)
{
	if (!InDrawData.HoverState.HoveredCell)
	{
		return;
	}

	const auto[TopLeft, TopRight, BottomRight, BottomLeft] = GetCellIndices(*InDrawData.HoverState.HoveredCell, InDrawData.MatrixWidth);
	FVector2D Vert_TopLeft = InDrawData.ControlPoints[TopLeft];
	FVector2D Vert_TopRight = InDrawData.ControlPoints[TopRight];
	FVector2D Vert_BottomRight = InDrawData.ControlPoints[BottomRight];
	FVector2D Vert_BottomLeft = InDrawData.ControlPoints[BottomLeft];
	InsetQuadBy(Vert_TopLeft, Vert_TopRight, Vert_BottomRight, Vert_BottomLeft, LatticeDrawConstants::CellHoverInset);
	
	const FVector2f TexturePos(0.f, 0.f);
	const FSlateRenderTransform& Transform = InViewContainerGeometry.GetAccumulatedRenderTransform();
	const FColor TintColor = FLinearColor::White.CopyWithNewOpacity(LatticeDrawConstants::EdgeHighlightAlpha).ToFColor(true);
	const TArray Vertices {
		FSlateVertex::Make<ESlateVertexRounding::Disabled>(Transform, FVector2f(Vert_TopLeft.X, Vert_TopLeft.Y), TexturePos, TintColor),
		FSlateVertex::Make<ESlateVertexRounding::Disabled>(Transform, FVector2f(Vert_TopRight.X, Vert_TopRight.Y), TexturePos, TintColor),
		FSlateVertex::Make<ESlateVertexRounding::Disabled>(Transform, FVector2f(Vert_BottomRight.X, Vert_BottomRight.Y), TexturePos, TintColor),
		FSlateVertex::Make<ESlateVertexRounding::Disabled>(Transform, FVector2f(Vert_BottomLeft.X, Vert_BottomLeft.Y), TexturePos, TintColor)
	};
	
	// Handle concave quad: If the quad is concave, then the diagonal of the 2 triangles must not be on the vertex that has the concave angle.
	// Note: this does not handle crossed quads (i.e. where the "quad" is 2 triangles with 2 "opposite" edges crossing each other).
	const float AngleTopLeft = FMath::Acos(FVector2D::DotProduct(Vert_BottomLeft - Vert_TopLeft, Vert_TopRight - Vert_TopLeft));
	const float AngleBottomRight = FMath::Acos(FVector2D::DotProduct(Vert_BottomLeft - Vert_BottomRight, Vert_TopRight - Vert_BottomRight));
	const bool bRotate = AngleTopLeft >= FMath::DegreesToRadians(180.f) || AngleBottomRight >= FMath::DegreesToRadians(180.f);
	const TArray<SlateIndex> Indices = bRotate ? TArray<SlateIndex>{ 0, 1, 2, 0, 2, 3 } : TArray<SlateIndex>{ 1, 2, 3, 3, 0, 1};

	const FSlateResourceHandle ResourceHandle = FSlateApplication::Get().GetRenderer()->GetResourceHandle(
		*FAppStyle::GetBrush("Sequencer.SectionArea.Background") // This is completely white so we can just multiply it with the TintColor
		);
	FSlateDrawElement::MakeCustomVerts(
		OutDrawElements, InPaintOnLayerId, ResourceHandle, Vertices, Indices, nullptr, 0, 0
		);
}

void DrawLatticeControls(
	const FLatticeControlsDrawData& InDrawData,
	const FGeometry& InViewContainerGeometry,
	FSlateWindowElementList& OutDrawElements,
	int32 InPaintOnLayerId
	)
{
	// Cells highlight first...
	DrawHoveredCells(InDrawData, InViewContainerGeometry, OutDrawElements, InPaintOnLayerId);
	// ... as edges are drawn over cells...
	DrawControlEdges(InDrawData, InViewContainerGeometry, OutDrawElements, InPaintOnLayerId);
	// ... and control points draw over edges.
	DrawControlPoints(InDrawData, InViewContainerGeometry, OutDrawElements, InPaintOnLayerId);
}
}