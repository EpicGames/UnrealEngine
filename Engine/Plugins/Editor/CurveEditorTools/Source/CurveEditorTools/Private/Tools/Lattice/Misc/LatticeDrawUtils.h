// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "HAL/Platform.h"
#include "Math/Vector2D.h"

struct FCurveModelID;
class FCurveEditor;
class FSlateLayoutTransform;
class FSlateWindowElementList;
class SCurveEditorView;
struct FGeometry;
struct FKeyHandle;

/** Put utils specific to drawing the lattice controls into this file */
namespace UE::CurveEditorTools
{
struct FLatticeControlEdge;
struct FLatticeControlsDrawData;
struct FLatticeHoverState;

/**
 * Obtains the transform required to transform evaluated key positions to that space used by ICurveEditorToolExtension::OnPaint.
 * The widget hierarchy is SCurveEditorViewContainer -> SCurveEditorView.
 * - SCurveEditorView is used to evaluate key positions.
 * - ICurveEditorToolExtension::OnPaint receives the FGeometry of SCurveEditorView
 */
FSlateLayoutTransform ComputeViewToViewContainerTransform(const SCurveEditorView& InView, const FCurveEditor& InCurveEditor);
	
struct FLatticeBounds
{
	/** If false, then do not discard the other values. */
	bool bIsVisible = false;

	// Absolute key values, i.e. FKeyPosition::InputValue, FKeyPosition::OutputValue.
	FVector2D MinValues;
	FVector2D MaxValues;

	// Values on screen, but transformed to screen space.
	FVector2D MinSlatePosition;
	FVector2D MaxSlatePosition;

	// Axis values in curve space of the current view mode.
	// For example, in normalized mode, where the Y-axis only has values from 0 to 1, a Y could be 0.8, etc.
	FVector2d MinValuesCurveSpace;
	FVector2d MaxValuesCurveSpace;

	static FLatticeBounds MakeCombined(const FLatticeBounds& InFirst, const FLatticeBounds& InOther);
};

/** Computes the min and max slate and value points in the user's selection. */
FLatticeBounds ComputeBounds(const FCurveEditor& InCurveEditor);
/** Computes the bounds just for the given curve. */
FLatticeBounds ComputeCurveBounds(const FCurveEditor& InCurveEditor, const FCurveModelID& InCurveId, TConstArrayView<FKeyHandle> InKeys);
	
/**
 * Transforms the values of the control points to where they should be drawn in InLatticeGeometry.
 * @param InCurveEditor Needed to get interactive view for transforming values.
 * @param InBounds The bounds as computed by ComputeBounds
 * @param InLatticeGeometry The result of MakeLatticeGeometry
 * @param InControlPoints The points to transform
 * @param OutTransformedToViewContainer Will contain the transformed points; must be equal length to InControlPoints.
 * @return Whether OutTransformed was written to.
 */
bool TransformViewToViewContainer(
	const FCurveEditor& InCurveEditor,
	TConstArrayView<FVector2D> InControlPoints,
	TArrayView<FVector2D> OutTransformedToViewContainer
	);

/**
 * Computes which elements should be hovered.
 * @param InViewContainerGeometry The result of MakeLatticeGeometry
 * @param InMouseScreenPosition The screen space position of the mouse
 * @param InControlPoints The control points of the lattice deformer, already transformed using TransformControlPointsToSlate.
 * @param InControlEdges The edges of the lattice deformer.
 * @param InNumPointsInX The number of points InControlPoints has in x direction
 * @param InNumCells Total number of cells in the deformer
 */
FLatticeHoverState ComputeLatticeHoverState(
	const FGeometry& InViewContainerGeometry,
	const FVector2D& InMouseScreenPosition,
	TConstArrayView<FVector2D> InControlPoints,
	TConstArrayView<FLatticeControlEdge> InControlEdges,
	int32 InNumPointsInX,
	int32 InNumCells
	);

/**
 * Draws the lattice controls.
 * @param InDrawData 
 * @param InViewContainerGeometry The result of MakeLatticeGeometry
 * @param OutDrawElements 
 * @param InPaintOnLayerId 
 */
void DrawLatticeControls(
	const FLatticeControlsDrawData& InDrawData, const FGeometry& InViewContainerGeometry, FSlateWindowElementList& OutDrawElements, int32 InPaintOnLayerId
	);
}
