// Copyright Epic Games, Inc. All Rights Reserved.

#include "LatticeUtils.h"

#include "CurveEditor.h"
#include "CurveEditorSelection.h"
#include "CurveEditorTypes.h"
#include "Math/NumericLimits.h"
#include "Tools/Lattice/LatticeDeformer2D.h"

#include <type_traits>

namespace UE::CurveEditorTools
{
int32 MatrixIndicesToFlatIndex(int32 InColumn, int32 InRow, int32 InNumColumns)
{
	return InColumn + InRow * InNumColumns;
}

FMatrixIndices FlatIndexToMatrixIndices(int32 InFlatIndex, int32 InNumColumns)
{
	const int32 Col = InFlatIndex % InNumColumns;
	const int32 Row = InFlatIndex / InNumColumns;
	return FMatrixIndices{ Col, Row };
}

FCellCornerIndices GetMatrixCellSlotsWithCorner(int32 InCornerIndex, int32 InNumLatticePointColumns, int32 InNumLatticePointRows)
{
	const auto[Column, Row] = FlatIndexToMatrixIndices(InCornerIndex, InNumLatticePointColumns);

	FCellCornerIndices Result;
	const int32 MaxColumnIndex = InNumLatticePointColumns - 1;
	const int32 MaxRowIndex = InNumLatticePointRows - 1;
	
	const bool bHasTopLeftCell = Column > 0 && Row > 0;
	const bool bHasTopRightCell = Column < MaxColumnIndex && Row > 0;
	const bool bHasBottomRightCell = Column < MaxColumnIndex && Row < MaxRowIndex;
	const bool bHasBottomLeftCell = Column > 0 && Row < MaxRowIndex;

	Result.TopLeft = bHasTopLeftCell			? MatrixIndicesToFlatIndex(Column - 1	, Row - 1		, InNumLatticePointColumns - 1) : INDEX_NONE;
	Result.TopRight = bHasTopRightCell			? MatrixIndicesToFlatIndex(Column		, Row - 1		, InNumLatticePointColumns - 1) : INDEX_NONE;
	Result.BottomLeft = bHasBottomLeftCell		? MatrixIndicesToFlatIndex(Column - 1	, Row			, InNumLatticePointColumns - 1) : INDEX_NONE;
	Result.BottomRight = bHasBottomRightCell	? MatrixIndicesToFlatIndex(Column 		, Row			, InNumLatticePointColumns - 1) : INDEX_NONE;

	return Result;
}

FEdgeVertexIndices GetEdgeIndices(int32 InEdgeIndex, int32 InNumPointsInWidth, int32 InNumPointsInHeight)
{
	const int32 HorizontalEdges = (InNumPointsInWidth - 1) * InNumPointsInHeight;

	// Determine if the edge is horizontal
	if (InEdgeIndex < HorizontalEdges)
	{
		// Horizontal edge
		const int32 Row = InEdgeIndex / (InNumPointsInWidth - 1);            
		const int32 Col = InEdgeIndex % (InNumPointsInWidth - 1);          
		const int32 Left = Row * InNumPointsInWidth + Col;              
		const int32 Right = Row * InNumPointsInWidth + (Col + 1);        
		return { Left, Right };
	}
	else
	{
		// Vertical edge
		const int32 VerticalIndex = InEdgeIndex - HorizontalEdges;
		const int32 Column = VerticalIndex % InNumPointsInWidth;             
		const int32 Row = VerticalIndex / InNumPointsInWidth;             
		const int32 Top = Row * InNumPointsInWidth + Column;              
		const int32 Bottom = (Row + 1) * InNumPointsInWidth + Column;        
		return { Top, Bottom };
	}
}

FCellVertexIndices GetCellIndices(int32 InCellIndex, int32 InNumPointsInWidth)
{
	FCellVertexIndices CellIndices;
	
	const int32 CellWidth = InNumPointsInWidth - 1;
	const auto[CellColumn, CellRow] = FlatIndexToMatrixIndices(InCellIndex, CellWidth);
	
	CellIndices.TopLeft = MatrixIndicesToFlatIndex(CellColumn, CellRow, InNumPointsInWidth);
	CellIndices.TopRight = MatrixIndicesToFlatIndex(CellColumn + 1, CellRow, InNumPointsInWidth);
	CellIndices.BottomRight = MatrixIndicesToFlatIndex(CellColumn + 1, CellRow + 1, InNumPointsInWidth);
	CellIndices.BottomLeft = MatrixIndicesToFlatIndex(CellColumn, CellRow + 1, InNumPointsInWidth);

	return CellIndices;
}

bool IsLatticeTooSmall(const FVector2D& InMin, const FVector2D& InMax)
{
	const FVector2D DeltaValues = InMax - InMin;
	const bool bTooSmall = DeltaValues.X < UE_KINDA_SMALL_NUMBER || DeltaValues.Y < UE_KINDA_SMALL_NUMBER;
	return bTooSmall;
}
	
TArray<FVector2D> GenerateControlPoints(int32 InNumCellsInX, int32 InNumCellsInY, const FVector2D& InBottomLeft, const FVector2D& InTopRight)
{
	TArray<FVector2D> ControlPoints;
	// Lattice with 0 width or height does not make sense. 
	if (!ensure(InBottomLeft.X < InTopRight.X) || !ensure(InBottomLeft.Y < InTopRight.Y))
	{
		return ControlPoints;
	}

	const int32 NumPointsInWidth = InNumCellsInX + 1;
	const int32 NumPointsInHeight = InNumCellsInY + 1;
	const int32 NumControlPoints = NumPointsInWidth * NumPointsInHeight;
	if (NumControlPoints <= 0)
	{
		return ControlPoints;
	}
	ControlPoints.SetNumUninitialized(NumControlPoints);

	const double DeltaX = (InTopRight.X - InBottomLeft.X) / static_cast<double>(InNumCellsInX);
	const double DeltaY = (InTopRight.Y - InBottomLeft.Y) / static_cast<double>(InNumCellsInY);
	for (int32 X = 0; X < NumPointsInWidth; ++X)
	{
		for (int32 Y = 0; Y < NumPointsInWidth; ++Y)
		{
			const int32 Index = MatrixIndicesToFlatIndex(X, Y, NumPointsInWidth);
			const FVector2D ControlPosition(InBottomLeft.X + DeltaX * X, InBottomLeft.Y + DeltaY * Y);
			ControlPoints[Index] = ControlPosition;
		}
	}

	return ControlPoints;
}

TArray<FVector2D> TransformPoints(const FTransform2d& InTransform, TConstArrayView<FVector2D> InControlPoints)
{
	TArray<FVector2D> TransformedPoints;
	TransformedPoints.Reserve(InControlPoints.Num());
	Algo::Transform(InControlPoints, TransformedPoints, [&InTransform](const FVector2D& ControlPoint)
	{
		const FVector2D Transformed = InTransform.TransformPoint(ControlPoint);
		return Transformed;
	});
	return TransformedPoints;
}
}
