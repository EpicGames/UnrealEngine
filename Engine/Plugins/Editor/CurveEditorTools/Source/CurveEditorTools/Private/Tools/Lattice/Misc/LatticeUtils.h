// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/TransformCalculus2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreMiscDefines.h"
#include "HAL/Platform.h"

class FCurveEditor;
template<typename OptionalType> struct TOptional;

/** Put utils specific to lattice deforming into this file */
namespace UE::CurveEditorTools
{
template<typename T> class TLatticeDeformer2D;
struct FKeyInCurveId;
	
/** @return Index you can use in a TArray representing a 2D matrix. */
int32 MatrixIndicesToFlatIndex(int32 InColumn, int32 InRow, int32 InNumColumns);

struct FMatrixIndices
{
	int32 Column; // X
	int32 Row; // Y
};
/** @return Matrix Column and Row that InFlatIndex maps to if matrix is InNumColumns wide. Does not check bounds.*/
FMatrixIndices FlatIndexToMatrixIndices(int32 InFlatIndex, int32 InNumColumns);

struct FCellCornerIndices
{
	int32 TopLeft;
	int32 TopRight;
	int32 BottomRight;
	int32 BottomLeft;
};
/** Suppose you have nxm matrix of lattice points that forms a (n-1)x(m-1) matrix of cells, returns the flat indices of the cells that share InCornerIndex. */
FCellCornerIndices GetMatrixCellSlotsWithCorner(int32 InCornerIndex, int32 InNumLatticePointColumns, int32 InNumLatticePointRows);

struct FEdgeVertexIndices
{
	int32 Start = INDEX_NONE;
	int32 End = INDEX_NONE;
};
/**
 * @param InEdgeIndex Index of the edge according to how it is described in FLatticeDeformer2D.
 * @param InNumPointsInWidth The number of points in the x direction (num cells in x-direction + 1)
 * @param InNumPointsInHeight The number of points in the z direction (num cells in z-direction + 1)
 * @return The corner indices of the cell.
 */
FEdgeVertexIndices GetEdgeIndices(int32 InEdgeIndex, int32 InNumPointsInWidth, int32 InNumPointsInHeight);

struct FCellVertexIndices
{
	int32 TopLeft = INDEX_NONE;
	int32 TopRight = INDEX_NONE;
	int32 BottomRight = INDEX_NONE;
	int32 BottomLeft = INDEX_NONE;

	bool IsValid() const
	{
		// Either all or none are INDEX_NONE
		return TopLeft != INDEX_NONE && ensure(TopRight != INDEX_NONE) && ensure(BottomRight != INDEX_NONE) && ensure(BottomLeft != INDEX_NONE);
	}
};
/**
 * @param InCellIndex Index of the cell according to how it is described in FLatticeDeformer2D.
 * @param InNumPointsInWidth The number of points in the x direction (num cells in x-direction + 1)
 * @return The corner indices of the cell.
 */
FCellVertexIndices GetCellIndices(int32 InCellIndex, int32 InNumPointsInWidth);

enum class ELatticeEdgeType : uint8
{
	Top,
	Bottom,
	Left,
	Right,
};
/** @return Assuming the lattice grid consists of a single cell, gets the specified edge. */
constexpr int32 GetEdgeIndexInSingleCellGrid(ELatticeEdgeType Type);
/** @return Assuming the lattice grid consists of a single cell, gets type of edge based off of its index. */
constexpr ELatticeEdgeType GetEdgeTypeFromIndexInSingleCellGrid(int32 InEdgeIndex);
	
/** @return The opposite edge of the passed in edge */
constexpr ELatticeEdgeType GetOppositeEdge(ELatticeEdgeType Type);

/** @return Whether a lattice can be formed by these points. */
bool IsLatticeTooSmall(const FVector2D& InMin, const FVector2D& InMax);
	
/** @return The lattice control points given the construction arguments. Use GetCellIndices to access cell control points. */
TArray<FVector2D> GenerateControlPoints(int32 InNumCellsInX, int32 InNumCellsInY, const FVector2D& InBottomLeft, const FVector2D& InTopRight);

/** @return InControlPoints transformed by InTransform. */
TArray<FVector2D> TransformPoints(const FTransform2d& InTransform, TConstArrayView<FVector2D> InControlPoints);
}

namespace UE::CurveEditorTools
{
constexpr int32 GetEdgeIndexInSingleCellGrid(ELatticeEdgeType Type)
{
	// The enum entries are intentionally sorted for this op
	return static_cast<int32>(Type);
}

constexpr ELatticeEdgeType GetEdgeTypeFromIndexInSingleCellGrid(int32 InEdgeIndex)
{
	// The enum entries are intentionally sorted for this op
	return static_cast<ELatticeEdgeType>(InEdgeIndex);
}

constexpr ELatticeEdgeType GetOppositeEdge(ELatticeEdgeType Type)
{
	switch (Type)
	{
	case ELatticeEdgeType::Top: return ELatticeEdgeType::Bottom;
	case ELatticeEdgeType::Bottom: return ELatticeEdgeType::Top;
	case ELatticeEdgeType::Left: return ELatticeEdgeType::Right;
	case ELatticeEdgeType::Right: return ELatticeEdgeType::Left;
	default: return ELatticeEdgeType::Top;
	}
}
}
