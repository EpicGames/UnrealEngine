// Copyright Epic Games, Inc. All Rights Reserved.

#include "PerCurveLatticeData.h"

#include "Containers/Map.h"
#include "CurveEditor.h"
#include "LatticePointMetaData.h"
#include "SCurveEditorView.h"
#include "Misc/LatticeDrawUtils.h"
#include "Misc/LatticeUtils.h"
#include "Misc/VectorMathUtils.h"

namespace UE::CurveEditorTools
{
namespace PerCurveDetail
{
static void AddPointsToDeformer(TConstArrayView<FKeyHandle> InKeys, const FCurveModel* CurveModel, FPerCurveDeformer2D& LocalDeformer)
{
	// Looping like this avoids allocating a big FKeyPosition array.
	for (int32 Index = 0; Index < InKeys.Num(); ++Index)
	{
		const FKeyHandle& KeyId = InKeys[Index];
		
		FKeyPosition Position;
		TConstArrayView<FKeyHandle> SingleKeyView(&KeyId, 1);
		TArrayView<FKeyPosition> SinglePositionView(&Position, 1);
		CurveModel->GetKeyPositions(SingleKeyView, SinglePositionView);

		const FVector2D AsVector{ Position.InputValue, Position.OutputValue };
		const FLatticePointMetaData MetaData{ KeyId, Position };
		LocalDeformer.AddPoints_BeforeLatticeMoved(MakeConstArrayView(&MetaData, 1), MakeConstArrayView(&AsVector, 1));
	}
}
}
	
TMap<FCurveModelID, FPerCurveLatticeData> BuildPerLatticeData(const FLatticeBounds& Lattice, const FCurveEditor& InCurveEditor)
{
	TMap<FCurveModelID, FPerCurveLatticeData> PerCurveData;
	for (const TPair<FCurveModelID, FKeyHandleSet>& SelectionPair : InCurveEditor.Selection.GetAll())
	{
		const FCurveModelID& CurveId = SelectionPair.Key;
		const SCurveEditorView* View = InCurveEditor.FindFirstInteractiveView(CurveId);
		const FCurveModel* CurveModel = InCurveEditor.FindCurve(CurveId);
		if (!View || !CurveModel)
		{
			continue;
		}
		
		// 1. Need to convert Lattice.MinValuesCurveSpace & MaxValuesCurveSpace to absolute for this particular curve.
		const FTransform2d ViewToCurveTransform = View->GetViewToCurveTransform(CurveId);
		const FVector2D LocalMin = TransformCurveSpaceToAbsolute(ViewToCurveTransform, Lattice.MinValuesCurveSpace);
		const FVector2D LocalMax = TransformCurveSpaceToAbsolute(ViewToCurveTransform, Lattice.MaxValuesCurveSpace);
		// 2. Then, compute the transform to convert from NormalizedDeformer to LocalDeformer.
		const FTransform2d ToCurveSpace = TransformRectBetweenSpaces(Lattice.MinValues, Lattice.MaxValues, LocalMin, LocalMax);

		// In certain cases, the lattice may be too tiny after transformation (lattice maths would divide by zero - the constructor would ensure()).
		if (!IsLatticeTooSmall(LocalMin, LocalMax))
		{
			FPerCurveLatticeData& CurveData = PerCurveData.Add(
				CurveId, FPerCurveLatticeData{  FPerCurveDeformer2D(1, 1, LocalMin, LocalMax), ToCurveSpace }
			);
			PerCurveDetail::AddPointsToDeformer(SelectionPair.Value.AsArray(), CurveModel, CurveData.CurveDeformer);
		}
	}
	return PerCurveData;
}
}
