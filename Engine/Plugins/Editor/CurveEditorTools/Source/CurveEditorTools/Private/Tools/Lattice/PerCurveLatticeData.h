// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "LatticeDeformer2D.h"
#include "LatticeFwd.h"
#include "Math/TransformCalculus2D.h"
#include "Tools/Lattice/LatticePointMetaData.h"

class FCurveEditor;
struct FCurveModelID;
struct FKeyHandle;

namespace UE::CurveEditorTools
{
struct FLatticeBounds;

/** Data stored per curve in FCurveEditorLatticeTool. */
struct FPerCurveLatticeData
{
	/**
	 * Deformers for each curve; the control points are global deformer's control points transformed by the per-curve transform.
	 * The per-curve deformers' control points are always in absolute key space (i.e. exactly the values for FKeyPosition::InputValue and OutputValue).
	 */
	FPerCurveDeformer2D CurveDeformer;
	
	/**
	 * Transform from the global deformer's control points to per-curve lattice control points.
	 *
	 * The transform depends on the view:
	 * - In Absolute view, this is effectively an identity matrix.
	 * - For Normalized view, let's use an example:
	 *		- Suppose the following curves:
	 *			Curve A: global min and max absolute values are y=0 and y=100, respectively.
	 *			Curve B: global min and max absolute values are y=50 and y=150, respectively.
	 *		- Suppose the user selects keys such that Deformer is placed with bottom & top edge at y=0.6 & y=0.8 respectively.
	 *		- Deformer's bottom & top edge's absolute values would be y=60 and y=130 (--> the min & max absolute values across all selected curves).
	 *		- The per-curve lattices would be placed as follows
	 *			Curve A: lattice bottom & top edge at y=60 and y=80, respectively.
	 *			Curve B: lattice bottom & top edge at y=110 and y=130, respectively.
	 *		- The per-curve transforms are computed such that the Deformer's lattice with bottom & top edge at y=60 and y=130 and transformed to
	 *		the local lattices of Curve A (y=60 and y=80) and Curve B(y=110 and y=130), respectively.
	 *		
	 * The computation of the per-curve transform uses the per-curve transform of SCurveEditorView::GetViewToCurveTransform, which implements
	 * Absolute, Normalized, etc. views.
	 */
	FTransform2d GlobalDeformerToCurveDeformer;
};

/** Builds the per curve data for lattice. */
TMap<FCurveModelID, FPerCurveLatticeData> BuildPerLatticeData(const FLatticeBounds& Lattice, const FCurveEditor& InCurveEditor);
}
