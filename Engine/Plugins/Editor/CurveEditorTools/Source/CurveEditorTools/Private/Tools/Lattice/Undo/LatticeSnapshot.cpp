// Copyright Epic Games, Inc. All Rights Reserved.

#include "LatticeSnapshot.h"

#include "Algo/Transform.h"
#include "CurveEditor.h"
#include "SCurveEditorPanel.h"
#include "Tools/Lattice/Misc/LatticeUtils.h"

namespace UE::CurveEditorTools
{
FArchive& operator<<(FArchive& InArchive, FTransform2d& InTransform)
{
	const TMatrix2x2<double>& Matrix = InTransform.GetMatrix();
	FVector2D Translation = InTransform.GetTranslation();
	double M00, M01, M10, M11;
	Matrix.GetMatrix(M00, M01, M10, M11);
	
	InArchive << M00 << M01 << M10 << M11;
	InArchive << Translation;

	if (InArchive.IsLoading())
	{
		InTransform = FTransform2d(TMatrix2x2<double>(M00, M01, M10, M11), Translation);
	}
	return InArchive;
}
	
FArchive& operator<<(FArchive& InArchive, FPerCurveSnapshot& InSnapshot)
{
	InArchive << InSnapshot.GlobalToLocalTransform;
	InArchive << InSnapshot.KeyCoords;
	InArchive << InSnapshot.MetaData;
	return InArchive;
}
	
bool FLatticeSnapshot::CanApplySnapshot(const FCurveEditor& InCurveEditor) const
{
	return InCurveEditor.Selection.GetSerialNumber() == SelectionSerialNumber
		&& ViewMode && InCurveEditor.GetPanel() && InCurveEditor.GetPanel()->GetViewMode() == *ViewMode;
}
	
FArchive& operator<<(FArchive& InArchive, FLatticeSnapshot& InSnapshot)
{
	InArchive << InSnapshot.ViewMode;
	InArchive << InSnapshot.SelectionSerialNumber;
	InArchive << InSnapshot.ControlPointToCurveSpace;
	InArchive << InSnapshot.GlobalControlPoints;
	InArchive << InSnapshot.PerCurveData;
	return InArchive;
}

bool ApplySnapshot(
	const FLatticeSnapshot& InSnapshot, const FCurveEditor& InCurveEditor,
	FTransform2d& OutControlPointToCurveSpace, FGlobalLatticeDeformer2D& OutGlobalDeformer,
	TMap<FCurveModelID, FPerCurveLatticeData>& OutPerCurveData
)
{
	if (!InSnapshot.CanApplySnapshot(InCurveEditor)
		// To make implementation easier, we just handle 1 celled lattices for now. If you change that in the future, adjust the below code.
		|| !ensure(OutGlobalDeformer.NumCells() == 1))
	{
		return false;
	}

	// Properties for global deformers
	OutControlPointToCurveSpace = InSnapshot.ControlPointToCurveSpace;
	for (int32 PointIndex = 0; PointIndex < InSnapshot.GlobalControlPoints.Num(); ++PointIndex)
    {
    	TConstArrayView<FVector2D> ControlPoint(&InSnapshot.GlobalControlPoints[PointIndex], 1);
    	OutGlobalDeformer.SetControlsPointNoRecompute({ PointIndex }, { ControlPoint });
    }

	// Per-curve data
	OutPerCurveData.Empty(InSnapshot.PerCurveData.Num());
    for (const TPair<FCurveModelID, FPerCurveSnapshot>& Pair : InSnapshot.PerCurveData)
    {
    	const FPerCurveSnapshot& Snapshot = Pair.Value;
    	const TArray<FVector2D> ControlPoints = TransformPoints(Snapshot.GlobalToLocalTransform, OutGlobalDeformer.GetControlPoints());
    	constexpr int32 CellIndexZero = 0;
    	const auto[TopLeft, TopRight, BottomRight, BottomLeft] = GetCellIndices(CellIndexZero, 2/*NumPoints*/);
    	
    	const FVector2D Min = FVector2D::Min(FVector2D::Min(ControlPoints[0], ControlPoints[1]), FVector2D::Min(ControlPoints[2], ControlPoints[3]));
    	const FVector2D Max = FVector2D::Max(FVector2D::Max(ControlPoints[0], ControlPoints[1]), FVector2D::Max(ControlPoints[2], ControlPoints[3]));
    	// This *might* happen but be really irrealistic
    	if (IsLatticeTooSmall(Min, Max))
    	{
    		continue;
    	}
    	
    	FPerCurveLatticeData Data
    	{
    		.CurveDeformer = FPerCurveDeformer2D(1, 1, Min, Max),
    		.GlobalDeformerToCurveDeformer = Snapshot.GlobalToLocalTransform
    	};

    	FPerCurveDeformer2D& Deformer = Data.CurveDeformer;
    	Deformer.SetControlsPointNoRecompute(
    		{ TopLeft, TopRight, BottomRight, BottomLeft },
    		{ ControlPoints[TopLeft], ControlPoints[TopRight], ControlPoints[BottomRight], ControlPoints[BottomLeft] }
    		);
    	Deformer.SetKeyCoordsInCell(CellIndexZero, Snapshot.KeyCoords);
    	Deformer.SetCellMetaData(CellIndexZero, Snapshot.MetaData);
    	
    	OutPerCurveData.Add(Pair.Key, MoveTemp(Data));
    }
	
	return true;
}

FLatticeSnapshot TakeSnapshot(
	const FCurveEditor& InCurveEditor,
	const FTransform2d& InControlPointToCurveSpace, const FGlobalLatticeDeformer2D& InGlobalDeformer,
	const TMap<FCurveModelID, FPerCurveLatticeData>& InPerCurveData
	)
{
	FLatticeSnapshot Snapshot
	{
		.ViewMode = InCurveEditor.GetPanel() ? InCurveEditor.GetPanel()->GetViewMode() : TOptional<ECurveEditorViewID>(),
		.SelectionSerialNumber = InCurveEditor.Selection.GetSerialNumber(),
		.ControlPointToCurveSpace = InControlPointToCurveSpace,
		.GlobalControlPoints = TArray<FVector2D>(InGlobalDeformer.GetControlPoints())
	};

	Algo::Transform(InPerCurveData, Snapshot.PerCurveData, [](const TPair<FCurveModelID, FPerCurveLatticeData>& Pair)
	{
		FPerCurveSnapshot CurveSnapshot
		{
			.GlobalToLocalTransform = Pair.Value.GlobalDeformerToCurveDeformer,
			.KeyCoords = TArray<FLatticeKeyCoords>(Pair.Value.CurveDeformer.GetKeyCoordsInCell(0)),
			.MetaData = TArray<FLatticePointMetaData>(Pair.Value.CurveDeformer.GetCellMetaData(0))
		};
		return TPair<FCurveModelID, FPerCurveSnapshot>(Pair.Key, MoveTemp(CurveSnapshot));
	});
	
	return Snapshot;
}
}