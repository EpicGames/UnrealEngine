// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditorTypes.h"
#include "Misc/Optional.h"
#include "Tools/Lattice/LatticeFwd.h"
#include "Tools/Lattice/LatticePointMetaData.h"
#include "Tools/Lattice/PerCurveLatticeData.h"

class FCurveEditor;
enum class ECurveEditorViewID : uint64;

namespace UE::CurveEditorTools
{
struct FPerCurveSnapshot
{
	FTransform2d GlobalToLocalTransform;
	TArray<FLatticeKeyCoords> KeyCoords;
	/** The metadata for each point in the cell. */
	TArray<FLatticePointMetaData> MetaData;
	
	friend FArchive& operator<<(FArchive& InArchive, FPerCurveSnapshot& InSnapshot);
};
	
struct FLatticeSnapshot
{
	/** The view mode the editor was in when during this snapshot. The snapshot can only be applied if the editor is currently in the same view mode. */
	TOptional<ECurveEditorViewID> ViewMode;
	/** The selection serial number the editor had when this snapshot was taken. The snapshot can only be applied the selection states are the same. */
	uint32 SelectionSerialNumber;

	/** Corresponds to FLatticeDeformerState::ControlPointToCurveSpace. */
	FTransform2d ControlPointToCurveSpace;
	/** The controls points the global lattice had. Corresponds to FLatticeDeformerState::GlobalDeformer. */
	TArray<FVector2D> GlobalControlPoints;
	/** Corresponds to FLatticeDeformerState::PerCurveData. */
	TMap<FCurveModelID, FPerCurveSnapshot> PerCurveData;
	
	/** @return Whether the saved data is compatible with the currentl state of InCurveEditor. */
	bool CanApplySnapshot(const FCurveEditor& InCurveEditor) const;

	friend FArchive& operator<<(FArchive& InArchive, FLatticeSnapshot& InSnapshot);
};

/**
 * Tries to apply the snapshot if possible.
 * @return Whether the data was applied.
 */
bool ApplySnapshot(
	const FLatticeSnapshot& InSnapshot, const FCurveEditor& InCurveEditor,
	FTransform2d& OutControlPointToCurveSpace, FGlobalLatticeDeformer2D& OutGlobalDeformer,
	TMap<FCurveModelID, FPerCurveLatticeData>& OutPerCurveData
	);

/** Takes a snapshot of the lattice tool and the curve editor's data required in the future to validate that the data can be applied. */
FLatticeSnapshot TakeSnapshot(
	const FCurveEditor& InCurveEditor,
	const FTransform2d& InControlPointToCurveSpace, const FGlobalLatticeDeformer2D& InGlobalDeformer,
	const TMap<FCurveModelID, FPerCurveLatticeData>& InPerCurveData
	);
}

