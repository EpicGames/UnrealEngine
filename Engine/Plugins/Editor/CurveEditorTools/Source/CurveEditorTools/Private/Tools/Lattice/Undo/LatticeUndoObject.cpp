// Copyright Epic Games, Inc. All Rights Reserved.

#include "LatticeUndoObject.h"

#include "CurveEditor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LatticeUndoObject)

void UCurveEditorTools_LatticeUndoObject::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << Snapshot;
}

void UCurveEditorTools_LatticeUndoObject::PostEditUndo()
{
	UObject::PostEditUndo();
	OnPostEditUndo.Broadcast();
}
