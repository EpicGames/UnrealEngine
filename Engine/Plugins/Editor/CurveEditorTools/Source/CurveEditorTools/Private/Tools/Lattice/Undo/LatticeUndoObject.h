// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LatticeSnapshot.h"
#include "Delegates/Delegate.h"
#include "UObject/Object.h"
#include "LatticeUndoObject.generated.h"

/** Stores data for the purposes of undo / redo in the lattice tool. */
UCLASS()
class UCurveEditorTools_LatticeUndoObject : public UObject
{
	GENERATED_BODY()
public:

	DECLARE_MULTICAST_DELEGATE(FOnPostEditUndo);
	/** Invoked when the user has performed an undo / redo operation. You should call RestoreState. */
	FOnPostEditUndo OnPostEditUndo;

	/** Data that is undone / redone by the transaction sytem. */
	UE::CurveEditorTools::FLatticeSnapshot Snapshot;

	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostEditUndo() override;
	//~ End UObject Interface
};
