// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorRetimeTool.h"
#include "Async/ParallelFor.h"
#include "CurveEditorToolCommands.h"
#include "CurveEditor.h"
#include "CurveEditorCommands.h"
#include "Framework/Commands/UICommandList.h"
#include "ICurveEditorBounds.h"
#include "Rendering/DrawElementPayloads.h"
#include "ScopedTransaction.h"
#include "CurveModel.h"
#include "EditorFontGlyphs.h"
#include "Rendering/SlateRenderer.h"
#include "Fonts/FontMeasure.h"
#include "SCurveEditorView.h"
#include "Modification/Utils/ScopedCurveChange.h"
#include "Styling/CoreStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveEditorRetimeTool)

#define LOCTEXT_NAMESPACE "CurveEditorToolCommands"
namespace UE::CurveEditorTools
{
FCurveEditorRetimeTool::FCurveEditorRetimeTool(TWeakPtr<FCurveEditor> InCurveEditor)
	: WeakCurveEditor(InCurveEditor)
	, RetimeData(nullptr)
{
}

FCurveEditorRetimeTool::~FCurveEditorRetimeTool()
{
	if (RetimeData)
	{
		RetimeData->RemoveFromRoot();
	}
	RetimeData = nullptr;
}

FReply FCurveEditorRetimeTool::OnKeyDown(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	bScrubbing = IsScrubTimeKeyEvent(InKeyEvent);

	return FReply::Unhandled();
}

FReply FCurveEditorRetimeTool::OnKeyUp(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	bScrubbing = false;

	return FReply::Unhandled();
}

FReply FCurveEditorRetimeTool::OnMouseButtonDown(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	using namespace Retime;
	
	const TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();

	if (CurveEditor.IsValid() && MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		bool bChangedState = false;
		const bool bAddToSelection = MouseEvent.IsShiftDown();
		const bool bRemoveFromSelection = MouseEvent.IsAltDown();

		// We need to know if they've hit an anchor before they modify the selection, as we only want to do a full clear a selection if they haven't hit an anchor.
		bool bHitSelectedAnchor = false;
		for(int32 AnchorIndex = 0; AnchorIndex < RetimeData->RetimingAnchors.Num(); AnchorIndex++)
		{
			FCurveEditorRetimeAnchor& Anchor = RetimeData->RetimingAnchors[AnchorIndex];

			FGeometry MoveOnlyBarGeometry, RetimeBarGeometry, CloseButtonGeometry;
			Anchor.GetHitGeometry(MyGeometry, *CurveEditor, MoveOnlyBarGeometry, RetimeBarGeometry, CloseButtonGeometry);

			bHitSelectedAnchor = (MoveOnlyBarGeometry.IsUnderLocation(MouseEvent.GetScreenSpacePosition()) && Anchor.bIsSelected) || (RetimeBarGeometry.IsUnderLocation(MouseEvent.GetScreenSpacePosition()) && Anchor.bIsSelected);
			bool bHitCloseButton = CloseButtonGeometry.IsUnderLocation(MouseEvent.GetScreenSpacePosition());

			// We do an early out if they're trying to remove an anchor.
			if (bHitCloseButton)
			{
				RetimeData->RetimingAnchors.RemoveAt(AnchorIndex);
				return FReply::Handled();
			}
			if (bHitSelectedAnchor)
			{
				break;
			}
		}

		// If they left click without specifically adding or removing from the selection then we want to clear
		// their current selection before trying to select anything new.
		if (!bHitSelectedAnchor && !bAddToSelection && !bRemoveFromSelection)
		{
			for (FCurveEditorRetimeAnchor& Anchor : RetimeData->RetimingAnchors)
			{
				Anchor.bIsSelected = false;
			}
		}

		EDragMode DragMode;
		for (FCurveEditorRetimeAnchor& Anchor : RetimeData->RetimingAnchors)
		{
			FGeometry MoveOnlyBarGeometry, RetimeBarGeometry, CloseButtonGeometry;
			Anchor.GetHitGeometry(MyGeometry, *CurveEditor, MoveOnlyBarGeometry, RetimeBarGeometry, CloseButtonGeometry);
			const FVector2D MouseLocation = MouseEvent.GetScreenSpacePosition();
			const bool bHitMoveSection = MoveOnlyBarGeometry.IsUnderLocation(MouseLocation);
			const bool bHitRetimeSection = RetimeBarGeometry.IsUnderLocation(MouseLocation);
			if (bHitMoveSection || bHitRetimeSection)
			{
				// Holding shift to add takes precedence over holding Alt to remove.
				if (bRemoveFromSelection && !bAddToSelection)
				{
					Anchor.bIsSelected = false;
				}
				else
				{
					Anchor.bIsSelected = true;
				}

				bChangedState = true;
				DragMode = bHitRetimeSection ? EDragMode::RetimeMode : EDragMode::MoveAnchorMode;

				break;
			}
		}

		// Don't start a drag operation if we didn't click on anything.
		if (bChangedState)
		{
			MouseDownOp.Emplace(
				FDelayedDrag(MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()), MouseEvent.GetEffectingButton()),
				DragMode 
			);
		}

		// Let scrubbing bubble through to the parent widget
		if (!bScrubbing)
		{
			// Handle the left mouse button so clicks don't bubble through to selection.
			return FReply::Handled().PreventThrottling();
		}
	}

	// Right Clicks need to go through so that panning the graph works
	return FReply::Unhandled();
}

FReply FCurveEditorRetimeTool::OnMouseMove(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// Update Hover States
	const TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return FReply::Unhandled();
	}

	for (FCurveEditorRetimeAnchor& Anchor : RetimeData->RetimingAnchors)
	{
		FGeometry MoveOnlyBarGeometry, RetimeBarGeometry, CloseButtonGeometry;
		Anchor.GetHitGeometry(MyGeometry, *CurveEditor, MoveOnlyBarGeometry, RetimeBarGeometry, CloseButtonGeometry);

		Anchor.bIsHighlighted = MoveOnlyBarGeometry.IsUnderLocation(MouseEvent.GetScreenSpacePosition()) || RetimeBarGeometry.IsUnderLocation(MouseEvent.GetScreenSpacePosition());	
		Anchor.bIsCloseBtnHighlighted = CloseButtonGeometry.IsUnderLocation(MouseEvent.GetScreenSpacePosition());
	}

	if (MouseDownOp.IsSet())
	{
		FReply Reply = FReply::Handled();

		if (MouseDownOp->DelayedDrag.IsDragging())
		{
			// FVector2D MouseDelta = MouseEvent.GetCursorDelta();
			OnDrag(MouseDownOp->DelayedDrag.GetInitialPosition(), MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()));
		}
		else if (MouseDownOp->DelayedDrag.AttemptDragStart(MouseEvent))
		{
			OnDragStart();
			// Steal the capture, as we're now the authoritative widget in charge of a mouse-drag operation
			Reply.CaptureMouse(OwningWidget);
		}

		return Reply;
	}

	return FReply::Unhandled();
}

FReply FCurveEditorRetimeTool::OnMouseButtonUp(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = FReply::Handled();
	if (MouseDownOp.IsSet())
	{
		if (MouseDownOp->DelayedDrag.IsDragging())
		{
			OnDragEnd();
			
			// Only return handled if we actually started a drag
			Reply.ReleaseMouseCapture();
		}

		MouseDownOp.Reset();
		return Reply;
	}

	return FReply::Unhandled();
}

FReply FCurveEditorRetimeTool::OnMouseButtonDoubleClick(TSharedRef<SWidget> OwningWidget, const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
		if (CurveEditor)
		{
			FVector2D LocalPosition = InMyGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition());
			FCurveEditorScreenSpaceH HorizontalTransform = CurveEditor->GetPanelInputSpace();
			double NewAnchorTime = HorizontalTransform.ScreenToSeconds(LocalPosition.X);

			const FScopedTransaction Transaction(LOCTEXT("AddRetimingAnchorTransaction", "Add Retiming Anchor"));

			RetimeData->Modify();
			RetimeData->RetimingAnchors.Add(FCurveEditorRetimeAnchor(NewAnchorTime));

			// Sort anchors to include the new one in right order.
			SortAnchorsByTime();
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

void FCurveEditorRetimeTool::OnFocusLost(const FFocusEvent& InFocusEvent)
{
	// We need to end our drag if we lose Window focus to close the transaction, otherwise alt-tabbing while dragging
	// can cause a transaction to get stuck open.
	StopDragIfPossible();
}

void FCurveEditorRetimeTool::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 PaintOnLayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	// Draw a box which tells them how to use the tool if they've removed all of the anchor points.
	if (RetimeData->RetimingAnchors.Num() == 0)
	{
		// Darken the background so our text stands out more
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			PaintOnLayerId,
			AllottedGeometry.ToPaintGeometry(),
			FAppStyle::GetBrush(TEXT("WhiteBrush")),
			ESlateDrawEffect::None,
			FLinearColor::Black.CopyWithNewOpacity(0.75f)
		);

		// Draw some text telling the user how to get retime anchors.
		const FText NoAnchorsText = LOCTEXT("RetimeToolNoAnchors", "Double click to create a Retime Anchor.");
		const FSlateFontInfo FontInfo = FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont");
		const FLinearColor LabelColor = FLinearColor::White;

		// We have to measure the string so we can draw it centered on the window. 
		const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		const FVector2D TextLabelSize = FontMeasure->Measure(NoAnchorsText, FontInfo);

		const FPaintGeometry LabelGeometry = AllottedGeometry.ToPaintGeometry(
			FSlateLayoutTransform(
				FVector2D((AllottedGeometry.GetLocalSize().X - TextLabelSize.X) / 2.f, AllottedGeometry.GetLocalSize().Y / 2)
			)
		);

		FSlateDrawElement::MakeText(
			OutDrawElements,
			PaintOnLayerId,
			LabelGeometry,
			NoAnchorsText,
			FontInfo,
			ESlateDrawEffect::None,
			LabelColor
		);
		return;
	}

	// Draw the Anchors and segments between them.
	for (int32 AnchorIndex = 0; AnchorIndex < RetimeData->RetimingAnchors.Num(); AnchorIndex++)
	{
		const FCurveEditorRetimeAnchor* NextAnchor = AnchorIndex < RetimeData->RetimingAnchors.Num() - 1 ? &RetimeData->RetimingAnchors[AnchorIndex + 1] : nullptr;
		DrawAnchor(RetimeData->RetimingAnchors[AnchorIndex], NextAnchor, Args, AllottedGeometry, MyCullingRect, OutDrawElements, PaintOnLayerId, InWidgetStyle, bParentEnabled);
	}
}

void FCurveEditorRetimeTool::DrawAnchor(const FCurveEditorRetimeAnchor& InAnchor, const FCurveEditorRetimeAnchor* OptionalNextAnchor, const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 PaintOnLayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return;
	}

	FGeometry MoveOnlyBarGeometry, RetimeBarGeometry, CloseButtonGeometry;
	InAnchor.GetPaintGeometry(AllottedGeometry, *CurveEditor, MoveOnlyBarGeometry, RetimeBarGeometry, CloseButtonGeometry);

	// A selected anchor is solid while a non-selected one is partially transparent
	FLinearColor AnchorColor = InAnchor.bIsSelected ? FLinearColor::Green : FLinearColor::Green.CopyWithNewOpacity(0.75f);

	// Highlighted anchors are brought more towards white. This is done after choosing the color as a handle can be
	// both selected and highlighted (so you know that it passed a hit-state check, ie: deselecting one)
	if (InAnchor.bIsHighlighted)
	{
		AnchorColor = FMath::Lerp(AnchorColor, FLinearColor::White, 0.10f);
	}

	// Draw the core box handle.
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		PaintOnLayerId + 2,
		MoveOnlyBarGeometry.ToPaintGeometry(),
		FAppStyle::GetBrush("WhiteBrush"),
		ESlateDrawEffect::None,
		AnchorColor
	);

	// Draw the thicker middle handle.
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		PaintOnLayerId + 2,
		RetimeBarGeometry.ToPaintGeometry(),
		FAppStyle::GetBrush("WhiteBrush"),
		ESlateDrawEffect::None,
		AnchorColor
	);

	// Draw the close button.
	{
		FLinearColor ButtonColor = InAnchor.bIsSelected ? FLinearColor::Green : FLinearColor::Green.CopyWithNewOpacity(0.75f);
		if (InAnchor.bIsCloseBtnHighlighted)
		{
			ButtonColor = FMath::Lerp(ButtonColor, FLinearColor::White, 0.10f);
		}

		const FSlateFontInfo FontInfo = FAppStyle::Get().GetFontStyle("FontAwesome.11");
		FSlateDrawElement::MakeText(
			OutDrawElements,
			PaintOnLayerId + 1,
			CloseButtonGeometry.ToPaintGeometry(),
			FEditorFontGlyphs::Times_Circle,
			FontInfo,
			ESlateDrawEffect::None,
			ButtonColor
		);
	}

	// Now we attempt to draw a 'background' between the anchors. This background is gradated based on the selection state
	// of the anchors - it'll fade off to gray where it doesn't affect keys. This doesn't apply to the last handle to be drawn,
	// we always try to draw from n to n+1.
	if (OptionalNextAnchor)
	{
		const FLinearColor SelectedGradientColor = FLinearColor::Green.CopyWithNewOpacity(0.25f);
		const FLinearColor UnselectedGradientColor = FLinearColor::Gray.CopyWithNewOpacity(0.25f);

		FGeometry NextMoveOnlyBarGeometry, NextRetimeBarGeometry, NextCloseButtonGeometry;
		OptionalNextAnchor->GetPaintGeometry(AllottedGeometry, *CurveEditor, NextMoveOnlyBarGeometry, NextRetimeBarGeometry, NextCloseButtonGeometry);

		// The width is measured from the top left corners. We subtract the width of one anchor (and then offset by that much) to not overlap the anchors.
		const double GradientWidth = (NextMoveOnlyBarGeometry.GetLocalPositionAtCoordinates(FVector2D(0,0)) - MoveOnlyBarGeometry.GetLocalPositionAtCoordinates(FVector2D(0,0))).X - AnchorMoveOnlyBarWidth;
		const FVector2D GradientSize = FVector2D(GradientWidth, NextMoveOnlyBarGeometry.GetLocalSize().Y);
		const FVector2D GradientOffset = FVector2D(MoveOnlyBarGeometry.GetLocalPositionAtCoordinates(FVector2D::ZeroVector) + FVector2D(AnchorMoveOnlyBarWidth, 0));
		
		const FGeometry GradientGeometry = AllottedGeometry.MakeChild(GradientSize, FSlateLayoutTransform(GradientOffset));

		TArray<FSlateGradientStop> GradientStops;
		GradientStops.Add(FSlateGradientStop(FVector2D::ZeroVector, InAnchor.bIsSelected ? SelectedGradientColor : UnselectedGradientColor));
		GradientStops.Add(FSlateGradientStop(GradientSize, OptionalNextAnchor->bIsSelected ? SelectedGradientColor : UnselectedGradientColor));

		FSlateDrawElement::MakeGradient(
			OutDrawElements,
			PaintOnLayerId,
			GradientGeometry.ToPaintGeometry(),
			GradientStops,
			EOrientation::Orient_Vertical);
	}
}

void FCurveEditorRetimeTool::OnToolActivated()
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor.IsValid())
	{
		return;
	}

	// We initialize two bounds by default to encompass almost the entire range for a better UX. This is only done the first time we activate the tool
	// so that it holds your selection between usages. This should always get called before any other events so it's okay that RetimeData isn't
	// initialized in the constructor.
	if (RetimeData == nullptr)
	{
		double LeftBound, RightBound;
		CurveEditor->GetBounds().GetInputBounds(LeftBound, RightBound);

		double TotalInputValue = RightBound - LeftBound;
		double MiddleInputValue = LeftBound + (TotalInputValue / 2.0);
		
		// Use 90% of the visible space
		TotalInputValue = TotalInputValue * 0.9;
		LeftBound = MiddleInputValue - (TotalInputValue / 2.0);
		RightBound = MiddleInputValue + (TotalInputValue / 2.0);		

		RetimeData = NewObject<UCurveEditorRetimeToolData>();
		RetimeData->SetFlags(RF_Transactional);
		RetimeData->AddToRoot();

		RetimeData->RetimingAnchors.Add(FCurveEditorRetimeAnchor(LeftBound));
		RetimeData->RetimingAnchors.Add(FCurveEditorRetimeAnchor(RightBound));
	}

	CachedSelectionSet.Reset();

	for (const TPair<FCurveModelID, FKeyHandleSet>& Pair : CurveEditor->GetSelection().GetAll())
	{
		CachedSelectionSet.Add(Pair);
	}

	CurveEditor->GetSelection().Clear();
}

void FCurveEditorRetimeTool::OnToolDeactivated()
{
	// Close our transaction if they switch away from the tool mid drag.
	StopDragIfPossible();

	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor.IsValid())
	{
		return;
	}

	// Restore the user's key selection
	for (const TPair<FCurveModelID, FKeyHandleSet>& Pair : CachedSelectionSet)
	{
		CurveEditor->GetSelection().Add(Pair.Key, ECurvePointType::Key, Pair.Value.AsArray());
	}
}

void FCurveEditorRetimeTool::BindCommands(TSharedRef<FUICommandList> CommandBindings)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (CurveEditor)
	{
		FIsActionChecked RetimeToolIsActive = FIsActionChecked::CreateSP(CurveEditor.ToSharedRef(), &FCurveEditor::IsToolActive, ToolID);
		FExecuteAction ActivateRetimeTool = FExecuteAction::CreateSP(CurveEditor.ToSharedRef(), &FCurveEditor::MakeToolActive, ToolID);

		CommandBindings->MapAction(FCurveEditorToolCommands::Get().ActivateRetimeTool, ActivateRetimeTool, FCanExecuteAction(), RetimeToolIsActive);
	}
}

FText FCurveEditorRetimeTool::GetLabel() const
{
	return FCurveEditorToolCommands::Get().ActivateRetimeTool->GetLabel();
}

FText FCurveEditorRetimeTool::GetDescription() const
{
	return FCurveEditorToolCommands::Get().ActivateRetimeTool->GetDescription();
}

FSlateIcon FCurveEditorRetimeTool::GetIcon() const
{
	return FCurveEditorToolCommands::Get().ActivateRetimeTool->GetIcon();
}

void FCurveEditorRetimeTool::OnDragStart()
{

	using namespace Retime;

	if (const TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin())
	{
		ActiveTransaction.Emplace(CurveEditor, LOCTEXT("CurveEditorRetimeToolTransaction", "Retime Key(s)"));
		// Store our Retime Data so the handles reset to the correct location upon undo/redo.
		RetimeData->Modify();

		// We need to store a lot of information about the current state of the editor so that when we do our transforms,
		// we can continually base information off of the starting state. Without this, you end up creating a tool which
		// isn't communicative within a click, drag, release session.
		FPreDragCurveData ResizeData;
		for (FCurveEditorRetimeAnchor& Anchor : RetimeData->RetimingAnchors)
		{
			ResizeData.RetimeAnchorStartTimes.Add(Anchor.ValueInSeconds);
		}

		TSet<FCurveModelID> SelectedCurves = CurveEditor->GetEditedCurves();

		// Add the key times for all of the keys on all channels
		for (const FCurveModelID& CurveID : SelectedCurves)
		{
			FCurveModel* CurveModel = CurveEditor->FindCurve(CurveID);
			check(CurveModel);

			FPreDragChannelData& ChannelData = ResizeData.CurveChannels[ResizeData.CurveChannels.Emplace(CurveID)];

			// Get all Key Handles
			CurveModel->GetAllKeys(ChannelData.Handles);

			// Get all Key Positions for those Handles.
			ChannelData.FrameNumbers.SetNumUninitialized(ChannelData.Handles.Num());
			CurveModel->GetKeyPositions(ChannelData.Handles, ChannelData.FrameNumbers);
			ChannelData.LastDraggedFrameNumbers = ChannelData.FrameNumbers;
		}

		// Store our new data for use in OnDrag
		MouseDownOp->PreDragCurveData = ResizeData;
	}
}

void FCurveEditorRetimeTool::OnDrag(const FVector2D& InStartPosition, const FVector2D& InMousePosition)
{
	using namespace Retime;
	const double DragDeltaFromStartPx = InMousePosition.X - InStartPosition.X;

	switch (MouseDownOp->CurrentDragMode)
	{
		// Anchors are sorted once on mouse up to avoid unnecessary operations since it has no effect during drag.
		case EDragMode::MoveAnchorMode: OnDrag_MoveAnchors(DragDeltaFromStartPx); break;
		// Anchors are sorted during drag because key positions depend on their current order.
		case EDragMode::RetimeMode: OnDrag_RetimeKeysAndAnchors(DragDeltaFromStartPx); break;
		default: checkNoEntry(); break;
	}
}

void FCurveEditorRetimeTool::OnDrag_RetimeKeysAndAnchors(const double DragDeltaFromStartPx)
{
	OnDrag_MoveAnchors(DragDeltaFromStartPx);
	const TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();

	if (CurveEditor)
	{
		// Profiled 2nd October, 2025, 5.7. See CL 46467607 description.
		ParallelFor(MouseDownOp->PreDragCurveData->CurveChannels.Num(),[this, CurveEditor, DragDeltaFromStartPx](const int32 ChannelIndex)
		{
			RecomputeLastDraggedFrameNumbersFromDrag(*CurveEditor, ChannelIndex, DragDeltaFromStartPx); 
		});

		// Need to loop through it again because SetKeyPositions is only safe to call on the game thread.
		for (Retime::FPreDragChannelData& ChannelData : MouseDownOp->PreDragCurveData->CurveChannels)
		{
			FCurveModel* Curve = CurveEditor->FindCurve(ChannelData.CurveID);
			if (!Curve)
			{
				continue;
			}

			// Profiled 2nd October, 2025, 5.7. See CL 46467607 description.
			const TArrayView<const FKeyHandle> Keys = ChannelData.Handles;
			const TArrayView<const FKeyPosition> Positions = ChannelData.LastDraggedFrameNumbers;
			Curve->SetKeyPositions(Keys, Positions, EPropertyChangeType::Interactive);	
		}
	}
	
	// Sort to ensure anchors are always sorted from least to greatest as drawing and between segments counts on this behavior.
	SortAnchorsByTime();
}

void FCurveEditorRetimeTool::RecomputeLastDraggedFrameNumbersFromDrag(
	const FCurveEditor& CurveEditor, const int32 ChannelIndex, const double DragDeltaFromStartPx
	)
{
	Retime::FPreDragChannelData& ChannelData = MouseDownOp->PreDragCurveData->CurveChannels[ChannelIndex];
	const SCurveEditorView* View = CurveEditor.FindFirstInteractiveView(ChannelData.CurveID);
	if (!View)
	{
		return;
	}

	FCurveModel* Curve = CurveEditor.FindCurve(ChannelData.CurveID);
	if (!ensureAlways(Curve))
	{
		return;
	}

	const FCurveEditorScreenSpace CurveSpace = View->GetCurveSpace(ChannelData.CurveID);
	const double DeltaInputValue = DragDeltaFromStartPx / CurveSpace.PixelsPerInput();
	
	// For each key on the channel we need to figure out which two anchors it's affected by so we can modify the position based on whether or not those anchors moved
	for (int32 KeyIndex = 0; KeyIndex < ChannelData.Handles.Num(); KeyIndex++)
	{
		const FKeyPosition& Position = ChannelData.FrameNumbers[KeyIndex];

		// We need to find the two nearest handles for the key. A key may fall outside of any handle ranges so there may not be one.
		// In this case we can treat the two handles as the same and give it a movement weight of 1.0. Otherwise we calculate the weight
		// based on the fraction of the distance between the two handles (unless both are selected at which point it's full movement.
		int32 LeftAnchorIndex = INDEX_NONE , RightAnchorIndex = INDEX_NONE ;

		const double OriginalKeyTime = Position.InputValue;
		
		for (int32 AnchorIndex = 0; AnchorIndex < MouseDownOp->PreDragCurveData->RetimeAnchorStartTimes.Num(); AnchorIndex++)
		{
			double OriginalAnchorTime = MouseDownOp->PreDragCurveData->RetimeAnchorStartTimes[AnchorIndex];

			// We know anchors are sorted from lowest value to highest value, so we need to find the highest anchor that doesn't go past our key time to find the left one.
			if (OriginalAnchorTime < OriginalKeyTime)
			{
				LeftAnchorIndex = AnchorIndex;
			}
			else if (OriginalAnchorTime > OriginalKeyTime)
			{
				// This means that this anchor is actually already to the right of the key, so we don't want to update our Left Key.
				break;
			}
		}

		// We can assume the right anchor will be the left anchor's index (since the array is sorted)... assuming there's enough keys. If there's not enough keys then both anchors end up with the same index.
		if (LeftAnchorIndex + 1 < RetimeData->RetimingAnchors.Num())
		{
			RightAnchorIndex = LeftAnchorIndex + 1;
		}
		else
		{
			RightAnchorIndex = LeftAnchorIndex;
		}

		// There's no guarantee that the anchor indexes are valid yet! If the Left Anchor index was never found (too far to the right of the key we're looking at) it'll be -1 still, but Right Anchor will be 0.
		// We know there's at least one retiming anchor (to be in this block of code) so we can assume if the left anchor wasn't found we can use index 0 for it as well.
		if (LeftAnchorIndex == INDEX_NONE )
		{
			LeftAnchorIndex = 0;
			RightAnchorIndex = 0;
		}

		// Okay, now we know both Left and Right indexes are valid. They may point to the same anchor, but that's okay. Now we need to calculate the amount of influence on this key.
		// If they do point to the same key, the math below doesn't actually end up mattering because the lerp goes between the same value, which is either full on or full off based on whether or not
		// the nearest key was selected.
		const double OriginalLeftAnchorTime = MouseDownOp->PreDragCurveData->RetimeAnchorStartTimes[LeftAnchorIndex];
		const double OriginalRightAnchorTime = MouseDownOp->PreDragCurveData->RetimeAnchorStartTimes[RightAnchorIndex];
		const double LeftAnchorInfluence = RetimeData->RetimingAnchors[LeftAnchorIndex].bIsSelected ? 1.0 : 0;
		const double RightAnchorInfluence = RetimeData->RetimingAnchors[RightAnchorIndex].bIsSelected ? 1.0 : 0;

		// Figure out the fraction [0-1] between the two anchors our key is.
		const double Fraction = FMath::GetMappedRangeValueClamped(FVector2D(OriginalLeftAnchorTime, OriginalRightAnchorTime), FVector2D(0, 1), OriginalKeyTime);

		// Now we can remap between the two influences based on fraction amount.
		const double InfluenceOnKey = FMath::Lerp(LeftAnchorInfluence, RightAnchorInfluence, Fraction);

		const double NewKeyValue = OriginalKeyTime + (DeltaInputValue * InfluenceOnKey);
		
		// Snap the new values to the grid.
		FKeyPosition NewPosition = Position;
		NewPosition.InputValue = View->IsTimeSnapEnabled() ? CurveEditor.GetCurveSnapMetrics(ChannelData.CurveID).SnapInputSeconds(NewKeyValue) : NewKeyValue;

		// Update our Curve Model with our updated data.				
		ChannelData.LastDraggedFrameNumbers[KeyIndex] = NewPosition;				
	}
}

void FCurveEditorRetimeTool::OnDrag_MoveAnchors(const double DragDeltaFromStartPx)
{
	TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();	
	if (!CurveEditor)
	{
		return;
	}
	
	const FCurveEditorScreenSpaceH HorizontalTransform = CurveEditor->GetPanelInputSpace();
	const double InputSpaceDelta = DragDeltaFromStartPx / HorizontalTransform.PixelsPerInput();

	// Move all of our selected anchors by the delta value.
	for (int32 AnchorIndex = 0; AnchorIndex < RetimeData->RetimingAnchors.Num(); AnchorIndex++)
	{
		FCurveEditorRetimeAnchor& Anchor = RetimeData->RetimingAnchors[AnchorIndex];
		if (Anchor.bIsSelected)
		{
			// Anchors don't respect snap settings because the individual keys below do, and because of the linear scaling on the keys
			// you might need a non-snapped anchor to get them to jump over as you expect.
			Anchor.ValueInSeconds = MouseDownOp->PreDragCurveData->RetimeAnchorStartTimes[AnchorIndex] + InputSpaceDelta;
		}
	}	
}

void FCurveEditorRetimeTool::OnDragEnd()
{
	using namespace Retime;

	const TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
	if (!CurveEditor)
	{
		return;
	}
	// For each curve we're affecting.
	for (FPreDragChannelData& ChannelData : MouseDownOp->PreDragCurveData->CurveChannels)
	{
		FCurveModel* Curve = CurveEditor->FindCurve(ChannelData.CurveID);
		if (!ensureAlways(Curve))
		{
			continue;
		}
		
		const TArrayView<const FKeyHandle> Keys = ChannelData.Handles;
		const TArrayView<const FKeyPosition> Positions = ChannelData.LastDraggedFrameNumbers;
		Curve->SetKeyPositions(Keys, Positions, EPropertyChangeType::ValueSet);	
	}
	
	if (MouseDownOp->CurrentDragMode == EDragMode::MoveAnchorMode)
	{
		// When adjusting anchor position only sort anchors at the end of a drag to avoid unnecessary operations during anchor movement.
		SortAnchorsByTime();
	}
	// This finalizes the transaction
	ActiveTransaction.Reset();
	MouseDownOp.Reset();
}

void FCurveEditorRetimeTool::StopDragIfPossible()
{
	if (MouseDownOp.IsSet())
	{
		if (MouseDownOp->DelayedDrag.IsDragging())
		{
			OnDragEnd();
		}

		MouseDownOp.Reset();
	}
}

bool FCurveEditorRetimeTool::IsScrubTimeKeyEvent(const FKeyEvent& InKeyEvent) const
{
	const FCurveEditorCommands& Commands = FCurveEditorCommands::Get();
	// Need to iterate through primary and secondary to make sure they are all pressed.
	for (uint32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++i)
	{
		EMultipleKeyBindingIndex ChordIndex = static_cast<EMultipleKeyBindingIndex>(i);
		const FInputChord& Chord = *Commands.ScrubTime->GetActiveChord(ChordIndex);
		const bool bIsMovingTimeSlider = Chord.IsValidChord() && InKeyEvent.GetKey() == Chord.Key;
		if (bIsMovingTimeSlider)
		{
			return true;
		}
	}
	return false;
}

void FCurveEditorRetimeTool::SortAnchorsByTime()
{
	Algo::SortBy(RetimeData->RetimingAnchors, &FCurveEditorRetimeAnchor::ValueInSeconds);
}
}
#undef LOCTEXT_NAMESPACE // "CurveEditorToolCommands"