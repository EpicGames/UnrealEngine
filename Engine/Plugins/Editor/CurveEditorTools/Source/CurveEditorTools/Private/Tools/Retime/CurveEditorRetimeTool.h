// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ICurveEditorToolExtension.h"
#include "CurveEditorSelection.h"
#include "Modification/Utils/ScopedCurveChange.h"
#include "ScopedTransaction.h"
#include "RetimeAnchor.h"
#include "RetimeDragData.h"
#include "Modification/Utils/ScopedSelectionChange.h"
#include "CurveEditorRetimeTool.generated.h"

class FCurveEditor;
struct FCurveModelID;
struct FKeyHandle;

UCLASS()
class UCurveEditorRetimeToolData : public UObject
{
	GENERATED_BODY()

public:
	// List of anchor points. Assumes they're in order from lowest input time to greatest.del
	
	UPROPERTY()
	TArray<FCurveEditorRetimeAnchor> RetimingAnchors;
};

namespace UE::CurveEditorTools
{
class FCurveEditorRetimeTool : public ICurveEditorToolExtension
{
public:
	FCurveEditorRetimeTool(TWeakPtr<FCurveEditor> InCurveEditor);
	~FCurveEditorRetimeTool();

	// ICurveEditorToolExtension
	virtual void OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 PaintOnLayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual FReply OnKeyDown(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnKeyUp(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnMouseButtonDown(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonDoubleClick(TSharedRef<SWidget> OwningWidget, const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnMouseButtonUp(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(TSharedRef<SWidget> OwningWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnFocusLost(const FFocusEvent& InFocusEvent) override;
	virtual void OnToolActivated() override;
	virtual void OnToolDeactivated() override;
	virtual void BindCommands(TSharedRef<FUICommandList> CommandBindings) override;
	virtual FText GetLabel() const override;
	virtual FText GetDescription() const override;
	virtual FSlateIcon GetIcon() const override;
	// ~ICurveEditorToolExtension

private:
	virtual void DrawAnchor(const FCurveEditorRetimeAnchor& InAnchor, const FCurveEditorRetimeAnchor* OptionalNextAnchor, const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 PaintOnLayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const;
	virtual void OnDragStart();
	/** Execute retiming or anchor movement depending on which part of the anchor the user clicked. */
	virtual void OnDrag(const FVector2D& InStartPosition, const FVector2D& InMousePosition);
	virtual void OnDragEnd();

	void StopDragIfPossible();

	/** Adjust the positions of keys by moving anchors.*/
	void OnDrag_RetimeKeysAndAnchors(const double DragDeltaFromStartPx);
	/** Adjust the position of anchors without adjusting the timing of the keys. */
	void OnDrag_MoveAnchors(const double DragDeltaFromStartPx);
	/** Sort all anchors by input time to ensure they're always sorted from least to greatest as drawing and between segments counts on this behaviour. */
	void SortAnchorsByTime();
	/** Calculate positions of keys on the curve after retiming and save them to ChannelData.LastDraggedFrameNumbers for setting the key positions in OnDrag_RetimeKeysAndAnchors. */
	void RecomputeLastDraggedFrameNumbersFromDrag(const FCurveEditor& CurveEditor, const int32 ChannelIndex, const double DragDeltaFromStartPx);

	/** Returns true if the key event is a scrub time key event */
	bool IsScrubTimeKeyEvent(const FKeyEvent& InKeyEvent) const;

private:
	/** Weak pointer back to the owning curve editor. */
	TWeakPtr<FCurveEditor> WeakCurveEditor;

	/** The retiming data for this tool. This is a UObject for Undo/Redo purposes. Created and added to the root set on construction of the tool. */
	UCurveEditorRetimeToolData* RetimeData;

	struct FTransaction
	{
		FScopedTransaction Transaction;
		CurveEditor::FScopedCurveChange KeyChange;

		explicit FTransaction(TWeakPtr<FCurveEditor> InCurveEditor, const FText& Description)
			: Transaction(Description)
			, KeyChange([InCurveEditor]
			{
				using namespace UE::CurveEditor;
				return FCurvesSnapshotBuilder(
					InCurveEditor, ECurveChangeFlags::KeyAttributes | ECurveChangeFlags::MoveKeysAndRemoveStackedKeys
					).TrackAllCurves();
			}())
		{}
	};
	/** The currently open transaction (if any) */
	TOptional<FTransaction> ActiveTransaction;

	/** Set when attempting to move a drag handle. */
	TOptional<Retime::FMouseDownData> MouseDownOp;

	/** Cached user selection for keys. We clear this when the tool is opened and restore it when they leave */
	TMap<FCurveModelID, FKeyHandleSet> CachedSelectionSet;

	/** True while scrubbing */
	bool bScrubbing = false;
};
}