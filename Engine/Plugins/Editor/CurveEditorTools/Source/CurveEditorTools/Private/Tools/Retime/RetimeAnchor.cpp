// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetimeAnchor.h"

namespace UE::CurveEditorTools
{
/** Calculate geometry for the thicker bar handle of this anchor that is used for moving it to retime the keys. */
static FGeometry GetRetimeBarGeometry(const FGeometry& InWidgetGeometry, const FCurveEditor& InCurveEditor, const double ValueInSeconds, const double ExtraHorizontalPadding) 
{
	using namespace UE::CurveEditorTools;

	const FCurveEditorScreenSpaceH HorizontalTransform = InCurveEditor.GetPanelInputSpace();

	const double AnchorPosition = HorizontalTransform.SecondsToScreen(ValueInSeconds);
	const double BarHeight = InWidgetGeometry.GetLocalSize().Y - (CloseButtonWidth + CloseButtonPadding);

	// Set the size of the retime bar as half the height of the move-only bar
	FVector2D BarRetimeSize = FVector2D(AnchorRetimeBarWidth + ExtraHorizontalPadding, BarHeight/2);
	// Horizontally center the retime bar on top of the move-only bar
	const double HorizontalOffset = (AnchorMoveOnlyBarWidth - ExtraHorizontalPadding - AnchorRetimeBarWidth)/2;
	// Vertically center the retime bar - offset is quarter of the height of the move-only bar (H/2)/2 = H/4
	const FVector2D BarRetimeOffset = FVector2D(AnchorPosition + HorizontalOffset, BarHeight/4);

	return InWidgetGeometry.MakeChild(BarRetimeSize, FSlateLayoutTransform(BarRetimeOffset));
}
	
/** Calculate the geometry for the thin bar of this anchor that is used for adjusting its position on the timeline without moving the keys. */
FGeometry GetMoveOnlyBarGeometry(const FGeometry& InWidgetGeometry, const FCurveEditor& InCurveEditor, const double ValueInSeconds, const double ExtraHorizontalPadding) 
{
	using namespace UE::CurveEditorTools;

	const FCurveEditorScreenSpaceH HorizontalTransform = InCurveEditor.GetPanelInputSpace();

	const double AnchorPosition = HorizontalTransform.SecondsToScreen(ValueInSeconds);
	const double BarHeight = InWidgetGeometry.GetLocalSize().Y - (CloseButtonWidth + CloseButtonPadding);

	FVector2D BarTopSize = FVector2D(AnchorMoveOnlyBarWidth + ExtraHorizontalPadding, BarHeight);
	// Horizontally center the retime bar on top of the move-only bar
	const double HorizontalOffset = -ExtraHorizontalPadding/2;
	const FVector2D BarTopOffset = FVector2D(AnchorPosition + HorizontalOffset, 0);
	
	return InWidgetGeometry.MakeChild(BarTopSize, FSlateLayoutTransform(BarTopOffset));
}

/** Calculate geometry for the button for deleting this anchor. */
static FGeometry GetCloseButtonGeometry(const FGeometry& InWidgetGeometry, const FCurveEditor& InCurveEditor, double ValueInSeconds) 
{
	using namespace UE::CurveEditorTools;

	const FCurveEditorScreenSpaceH HorizontalTransform = InCurveEditor.GetPanelInputSpace();

	const double AnchorPosition = HorizontalTransform.SecondsToScreen(ValueInSeconds);
	const double BarHeight = InWidgetGeometry.GetLocalSize().Y - (CloseButtonWidth + CloseButtonPadding);

	const FVector2D ButtonSize = FVector2D(CloseButtonWidth, CloseButtonWidth);
	const FVector2D ButtonOffset = FVector2D(AnchorPosition, BarHeight + CloseButtonPadding);

	return InWidgetGeometry.MakeChild(ButtonSize, FSlateLayoutTransform(ButtonOffset));
}
}

void FCurveEditorRetimeAnchor::GetPaintGeometry(const FGeometry& InWidgetGeometry, const FCurveEditor& InCurveEditor, FGeometry& OutMoveOnlyBarGeometry, FGeometry& OutRetimeBarGeometry, FGeometry& OutCloseButtonGeometry) const
{
	using namespace UE::CurveEditorTools;
	
	OutMoveOnlyBarGeometry = GetMoveOnlyBarGeometry(InWidgetGeometry, InCurveEditor, ValueInSeconds, 0.0);
	OutRetimeBarGeometry = GetRetimeBarGeometry(InWidgetGeometry, InCurveEditor, ValueInSeconds, 0.0);
	OutCloseButtonGeometry = GetCloseButtonGeometry(InWidgetGeometry, InCurveEditor, ValueInSeconds);
}

void FCurveEditorRetimeAnchor::GetHitGeometry(const FGeometry& InWidgetGeometry, const FCurveEditor& InCurveEditor, FGeometry& OutMoveOnlyBarGeometry, FGeometry& OutRetimeBarGeometry, FGeometry& OutCloseButtonGeometry) const
{
	using namespace UE::CurveEditorTools;

	OutMoveOnlyBarGeometry = GetMoveOnlyBarGeometry(InWidgetGeometry, InCurveEditor, ValueInSeconds, ExtraPadding);
	OutRetimeBarGeometry = GetRetimeBarGeometry(InWidgetGeometry, InCurveEditor, ValueInSeconds, ExtraPadding);
	OutCloseButtonGeometry = GetCloseButtonGeometry(InWidgetGeometry, InCurveEditor, ValueInSeconds);
}