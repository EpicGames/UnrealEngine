// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditor.h"
#include "RetimeAnchor.generated.h"

USTRUCT()
struct FCurveEditorRetimeAnchor
{
	GENERATED_BODY()

	FCurveEditorRetimeAnchor()
		: ValueInSeconds(0.0)
		, bIsSelected(false)
		, bIsHighlighted(false)
		, bIsCloseBtnHighlighted(false)
	{
	}

	FCurveEditorRetimeAnchor(double InSeconds)
		: ValueInSeconds(InSeconds)
		, bIsSelected(false)
		, bIsHighlighted(false)
		, bIsCloseBtnHighlighted(false)
	{
	}

	/** The time on the Timeline that this anchor is anchored at. */
	UPROPERTY()
	double ValueInSeconds;

	/** Is this anchor currently selected? */
	UPROPERTY()
	bool bIsSelected;

	/** Is this anchor currently highlighted? An anchor can be both selected and highlighted. */
	bool bIsHighlighted;

	/** Is the close button highlighted? */
	bool bIsCloseBtnHighlighted;

	/** Calculate the paint geometry for this anchor. */
	void GetPaintGeometry(const FGeometry& InWidgetGeometry, const FCurveEditor& InCurveEditor, FGeometry& OutMoveOnlyBarGeometry, FGeometry& OutRetimeBarGeometry, FGeometry& OutCloseButtonGeometry) const;
	/** Calculate the wider hitbox geometry for this anchor. */
	void GetHitGeometry(const FGeometry& InWidgetGeometry, const FCurveEditor& InCurveEditor, FGeometry& OutMoveOnlyBarGeometry, FGeometry& OutRetimeBarGeometry, FGeometry& OutCloseButtonGeometry) const;
};

namespace UE::CurveEditorTools
{
	constexpr double AnchorMoveOnlyBarWidth = 3.0;
	constexpr double AnchorRetimeBarWidth = 5.0;
	constexpr double CloseButtonWidth = 18.0;
	constexpr double CloseButtonPadding = 2.0;
	constexpr double HighlightOpacity = 0.75;
	const double ExtraPadding = 8.0;
}