// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveDataAbstraction.h"
#include "Framework/DelayedDrag.h"

namespace UE::CurveEditorTools::Retime
{
struct FPreDragChannelData
{
	explicit FPreDragChannelData(FCurveModelID InCurveModel) : CurveID(InCurveModel) {}

	/** Array of all the handles in the section at the start of the drag */
	TArray<FKeyHandle> Handles;
	/** Array of all the above handle's times, one per index of Handles */
	TArray<FKeyPosition> FrameNumbers;
	/** Array of all the above handle's last dragged times, one per index of Handles */
	TArray<FKeyPosition> LastDraggedFrameNumbers;
	/** The ID of the curve the data corresponds to. */
	FCurveModelID CurveID;
};
	
struct FPreDragCurveData
{
	/** Array of all the channels in the section before it was resized */
	TArray<FPreDragChannelData> CurveChannels;
	/** The times on the Timeline that anchors are anchored at. */
	TArray<double> RetimeAnchorStartTimes;
};

/** Type of operation to execute, determined by which part of the anchor the user clicked at the start of the drag. */
enum class EDragMode : uint8
{
	/** Moving anchors will retime key positions accordingly. */
	RetimeMode,
	/** MoveAnchorMode: Anchors can be moved without affecting key positions. */
	MoveAnchorMode
}; 

struct FMouseDownData
{
	/** Set if CurrentDragMode == RetimeMode and once DelayedDrag has been detected. */
	TOptional<FPreDragCurveData> PreDragCurveData;

	/** Started on mouse down if any part of an anchor was clicked. */
	FDelayedDrag DelayedDrag;
	
	/** Active drag behavior (retime keys or move anchors). */
	EDragMode CurrentDragMode;
	
	FMouseDownData(const FDelayedDrag& InDrag, const EDragMode InMode) : DelayedDrag(InDrag), CurrentDragMode(InMode) {}
}; 
}